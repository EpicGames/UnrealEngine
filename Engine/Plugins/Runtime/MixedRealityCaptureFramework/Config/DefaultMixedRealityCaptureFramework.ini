[CoreRedirects]

;
; 4.20 Redirects
;

; Plugin/Module redirects
+PackageRedirects=(OldName="/MixedRealityFramework/...", NewName="/MixedRealityCaptureFramework/", MatchWildcard=true)
+PackageRedirects=(OldName="/Script/MixedRealityFramework", NewName="/Script/MixedRealityCaptureFramework")
+PackageRedirects=(OldName="/Script/MixedRealityCalibration", NewName="/Script/MixedRealityCaptureCalibration")

; General framework redirects
+ClassRedirects=(OldName="MixedRealityUtilLibrary", NewName="/Script/MixedRealityCaptureFramework.MrcUtilLibrary")
+ClassRedirects=(OldName="MixedRealityFrameworkSettings", NewName="/Script/MixedRealityCaptureFramework.MrcFrameworkSettings")
+StructRedirects=(OldName="MRLensDistortion",NewName="/Script/MixedRealityCaptureFramework.MrcLensDistortion")
+ClassRedirects=(OldName="MixedRealityGarbageMatteActor", NewName="/Script/MixedRealityCaptureFramework.MrcGarbageMatteActor")
+ClassRedirects=(OldName="MixedRealityGarbageMatteCaptureComponent", NewName="/Script/MixedRealityCaptureFramework.MrcGarbageMatteActor")
+ClassRedirects=(OldName="MixedRealityBillboard", NewName="/Script/MixedRealityCaptureFramework.MixedRealityCaptureBillboard")
+ClassRedirects=(OldName="MixedRealityBillboardActor", NewName="/Script/MixedRealityCaptureFramework.MixedRealityCaptureBillboard")
+StructRedirects=(OldName="MRCaptureDeviceIndex",NewName="/Script/MixedRealityCaptureFramework.MrcVideoCaptureFeedIndex")
+ClassRedirects=(OldName="MRCaptureDeviceLibrary", NewName="/Script/MixedRealityCaptureFramework.MrcVidCaptureDeviceLibrary")
+ClassRedirects=(OldName="AsyncTask_OpenMRCaptureFeedBase", NewName="/Script/MixedRealityCaptureFramework.AsyncTask_OpenMrcVidCaptureFeedBase")
+ClassRedirects=(OldName="AsyncTask_OpenMRCaptureDevice", NewName="/Script/MixedRealityCaptureFramework.AsyncTask_OpenMrcVidCaptureDevice")
+ClassRedirects=(OldName="AsyncTask_OpenMRCaptureFeed", NewName="/Script/MixedRealityCaptureFramework.AsyncTask_OpenMrcVidCaptureFeed")

; Save data redirects
+StructRedirects=(OldName="MRLensCalibrationData",NewName="/Script/MixedRealityCaptureFramework.MrcLensCalibrationData")
+StructRedirects=(OldName="MRAlignmentSaveData",NewName="/Script/MixedRealityCaptureFramework.MrcAlignmentSaveData")
+StructRedirects=(OldName="GarbageMatteSaveData",NewName="/Script/MixedRealityCaptureFramework.MrcGarbageMatteSaveData")
+StructRedirects=(OldName="VideoProcessingParams",NewName="/Script/MixedRealityCaptureFramework.MrcVideoProcessingParams")
+StructRedirects=(OldName="MRCompositingSaveData",NewName="/Script/MixedRealityCaptureFramework.MrcCompositingSaveData")
+ClassRedirects=(OldName="MixedRealityCalibrationData", NewName="/Script/MixedRealityCaptureFramework.MrcCalibrationData")
+ClassRedirects=(OldName="MixedRealityConfigurationSaveGame", NewName="/Script/MixedRealityCaptureFramework.MrcCalibrationSaveGame")

; Calibration redirects
+FunctionRedirects=(OldName="SteamVRFunctionLibrary.SetTrackingSpace",NewName="HeadMountedDisplayFunctionLibrary.SetTrackingOrigin")
+ClassRedirects=(OldName="MRCalibrationUtilLibrary", NewName="/Script/MixedRealityCaptureCalibration.MrcCalibrationUtilLibrary")
+StructRedirects=(OldName="MRAlignmentSample",NewName="/Script/MixedRealityCaptureCalibration.MrcAlignmentSample")
+ClassRedirects=(OldName="MRCalibrationSnapshotComponent", NewName="/Script/MixedRealityCaptureCalibration.MrcCalibrationSnapshotComponent")
+ClassRedirects=(OldName="MROpenCVCalibrator", NewName="/Script/MixedRealityCaptureCalibration.MrcOpenCVCalibrator")

; Asset redirects

+PackageRedirects=(OldName="/MixedRealityFramework/M_MRCamSrcProcessing", NewName="/MixedRealityCaptureFramework/M_MrcVideoProcessing") 
+ObjectRedirects=(OldName="/MixedRealityFramework/M_MRCamSrcProcessing.M_MRCamSrcProcessing", NewName="/MixedRealityCaptureFramework/M_MrcVideoProcessing.M_MrcVideoProcessing") 
+PackageRedirects=(OldName="/MixedRealityFramework/GarbageMattePlane", NewName="/MixedRealityCaptureFramework/SM_GarbageMattePlane") 
+ObjectRedirects=(OldName="/MixedRealityFramework/GarbageMattePlane.GarbageMattePlane", NewName="/MixedRealityCaptureFramework/SM_GarbageMattePlane.SM_GarbageMattePlane") 
+PackageRedirects=(OldName="/MixedRealityFramework/GarbageMatteRuntimeMaterial", NewName="/MixedRealityCaptureFramework/M_GarbageMatte") 
+ObjectRedirects=(OldName="/MixedRealityFramework/GarbageMatteRuntimeMaterial.GarbageMatteRuntimeMaterial", NewName="/MixedRealityCaptureFramework/M_GarbageMatte.M_GarbageMatte") 
+PackageRedirects=(OldName="/MixedRealityFramework/MRCameraSource", NewName="/MixedRealityCaptureFramework/MrcVideoSource") 
+ObjectRedirects=(OldName="/MixedRealityFramework/MRCameraSource.MRCameraSource", NewName="/MixedRealityCaptureFramework/MrcVideoSource.MrcVideoSource") 
+PackageRedirects=(OldName="/MixedRealityFramework/T_MRCameraSource", NewName="/MixedRealityCaptureFramework/T_MrcVideoSource") 
+ObjectRedirects=(OldName="/MixedRealityFramework/T_MRCameraSource.T_MRCameraSource", NewName="/MixedRealityCaptureFramework/T_MrcVideoSource.T_MrcVideoSource") 
+PackageRedirects=(OldName="/MixedRealityFramework/T_MRGarbageMatteRenderTarget", NewName="/MixedRealityCaptureFramework/T_MrcGarbageMatteRenderTarget") 
+ObjectRedirects=(OldName="/MixedRealityFramework/T_MRGarbageMatteRenderTarget.T_MRGarbageMatteRenderTarget", NewName="/MixedRealityCaptureFramework/T_MrcGarbageMatteRenderTarget.T_MrcGarbageMatteRenderTarget") 
+PackageRedirects=(OldName="/MixedRealityFramework/T_MRRenderTarget", NewName="/MixedRealityCaptureFramework/T_MrcRenderTarget") 
+ObjectRedirects=(OldName="/MixedRealityFramework/T_MRRenderTarget.T_MRRenderTarget", NewName="/MixedRealityCaptureFramework/T_MrcRenderTarget.T_MrcRenderTarget") 