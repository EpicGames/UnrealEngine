{
	"FileVersion": 3,

	"FriendlyName": "Mixed Reality Capture Framework",
	"Version": 1,
	"VersionName": "1.0",
	
	"Description": "A simple framework that provides users a way to integrate mixed reality capture into their VR projects.",
	"Category": "Virtual Reality",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"SupportURL": "https://answers.unrealengine.com/",

	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": true,

  "Modules": [
    {
      "Name": "MixedRealityCaptureFramework",
      "Type": "Runtime",
      "LoadingPhase": "PostEngineInit",
      "PlatformAllowList": [
        "Win64",
        "Linux"
      ]
    }
  ],
  "Plugins": [
	{
		"Name": "OpenCVLensDistortion",
		"Enabled": true
	},
	{
		"Name": "XRBase",
		"Enabled": true
	}
  ]
}