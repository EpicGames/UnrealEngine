// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MixedRealityCaptureFramework : ModuleRules
{
	public MixedRealityCaptureFramework(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"MediaAssets"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"Media",
				"HeadMountedDisplay",
				"InputCore",
                "MediaUtils",
				"RenderCore",
				"OpenCVLensDistortion",
				"OpenCVHelper",
				"XRBase",
			}
		);

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.Add("EditorFramework");
			PrivateDependencyModuleNames.Add("UnrealEd");
		}
	}
}
