// Copyright Epic Games, Inc. All Rights Reserved.

#include "MixedRealityCaptureActor.h"
#include "MixedRealityCaptureComponent.h"
#include "Engine/Engine.h" // for GEngine
#include "Engine/LocalPlayer.h"
#include "GameFramework/PlayerController.h"
#include "MrcUtilLibrary.h"
#include "GameFramework/Pawn.h"
#include "ISpectatorScreenController.h"
#include "Engine/TextureRenderTarget2D.h"
#include "IXRTrackingSystem.h" // for GetHMDDevice()
#include "IHeadMountedDisplay.h" // for GetSpectatorScreenController()
#include "Misc/ConfigCacheIni.h"
#include "Components/StaticMeshComponent.h"
#include "UObject/ConstructorHelpers.h"
#include "Engine/CollisionProfile.h" // for UCollisionProfile::NoCollision_ProfileName
#include "Engine/StaticMesh.h"

#define LOCTEXT_NAMESPACE "MRCaptureActor"

/* MixedRealityCaptureActor_Impl
 *****************************************************************************/

namespace MixedRealityCaptureActor_Impl
{
	static bool AssignTargetPlayer(AMixedRealityCaptureActor* CaptureActor, const bool bAutoAttach);
}

static bool MixedRealityCaptureActor_Impl::AssignTargetPlayer(AMixedRealityCaptureActor* CaptureActor, const bool bAutoAttach)
{
	bool bSuccess = false;

	if (UWorld* TargetWorld = CaptureActor->GetWorld())
	{
		APawn* FallbackPlayer = nullptr;

		const TArray<ULocalPlayer*>& LocalPlayers = GEngine->GetGamePlayers(TargetWorld);
		for (ULocalPlayer* Player : LocalPlayers)
		{
			if (APlayerController* Controller = Player->GetPlayerController(TargetWorld))
			{
				APawn* PlayerPawn = Controller->GetPawn();
				if (FallbackPlayer == nullptr)
				{
					FallbackPlayer = PlayerPawn;
				}

				USceneComponent* TrackingOrigin = UMrcUtilLibrary::GetHMDRootComponent(PlayerPawn);
				if (TrackingOrigin)
				{
					bSuccess = CaptureActor->SetTargetPlayer(PlayerPawn, bAutoAttach ? TrackingOrigin : nullptr);
					if (bSuccess)
					{
						FallbackPlayer = nullptr;
						break;
					}
				}
			}
		}

		if (FallbackPlayer)
		{
			bSuccess = CaptureActor->SetTargetPlayer(FallbackPlayer, bAutoAttach ? FallbackPlayer->GetRootComponent() : nullptr);
		}
	}

	return bSuccess;
}

/* FMRCaptureAutoTargeter
 *****************************************************************************/

class FMRCaptureAutoTargeter : public FTickableGameObject
{
public:
	FMRCaptureAutoTargeter(AMixedRealityCaptureActor* Owner, const bool bAutoAttach);

public:
	//~ FTickableObjectBase interface

	virtual bool IsTickable() const override;
	virtual void Tick(float DeltaTime) override;
	virtual TStatId GetStatId() const override;

private:
	AMixedRealityCaptureActor* Owner;
	bool bAutoAttach;
};

FMRCaptureAutoTargeter::FMRCaptureAutoTargeter(AMixedRealityCaptureActor* InOwner, const bool bAutoAttachIn)
	: Owner(InOwner)
	, bAutoAttach(bAutoAttachIn)
{}


bool FMRCaptureAutoTargeter::IsTickable() const
{
	return Owner && (Owner->GetRootComponent()->GetAttachParent() == nullptr);
}

void FMRCaptureAutoTargeter::Tick(float /*DeltaTime*/)
{
	MixedRealityCaptureActor_Impl::AssignTargetPlayer(Owner, bAutoAttach);
}

TStatId FMRCaptureAutoTargeter::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FPlayerAttachment, STATGROUP_ThreadPoolAsyncTasks);
}

/* AMixedRealityCaptureActor
 *****************************************************************************/

AMixedRealityCaptureActor::AMixedRealityCaptureActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bAutoAttachToVRPlayer(true)
	, bAutoHidePlayer(true)
	, bHideAttachmentsWithPlayer(true)
	, bAutoBroadcast(true)
{
	RootComponent = CreateDefaultSubobject<USceneComponent>(TEXT("TrackingSpaceOrigin"));

	CaptureComponent = CreateDefaultSubobject<UMixedRealityCaptureComponent>(TEXT("CaptureComponent"));
	CaptureComponent->SetupAttachment(RootComponent);

#if WITH_EDITORONLY_DATA
	if (!IsRunningCommandlet())
#endif
#if WITH_EDITORONLY_DATA || !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	{
		static ConstructorHelpers::FObjectFinder<UStaticMesh> VisualizerMeshFinder(TEXT("/Engine/BasicShapes/Cone"));
		DebugVisualizerMesh = VisualizerMeshFinder.Object;
	}	
#endif
}

bool AMixedRealityCaptureActor::SetTargetPlayer(APawn* PlayerPawn, USceneComponent* AttachTo)
{
	ensure(!AttachTo || AttachTo->GetOwner() == PlayerPawn);

	if (TargetPlayer.IsValid())
	{
		TargetPlayer->OnDestroyed.RemoveDynamic(this, &AMixedRealityCaptureActor::OnTargetDestroyed);
		CaptureComponent->HiddenActors.Remove(TargetPlayer.Get());
		TargetPlayer.Reset();
	}

	bool bSuccess = true;
	if (AttachTo)
	{
		AttachToComponent(AttachTo, FAttachmentTransformRules::KeepRelativeTransform);
		bSuccess = RootComponent->IsAttachedTo(AttachTo);
	}

	TargetPlayer = PlayerPawn;
	if (bAutoHidePlayer)
	{
		CaptureComponent->HiddenActors.AddUnique(PlayerPawn);

		TArray<AActor*> PlayerAttachments;
		if (bHideAttachmentsWithPlayer)
		{
			PlayerPawn->GetAttachedActors(PlayerAttachments);
		}
		else
		{
			PlayerPawn->GetAllChildActors(PlayerAttachments);
		}

		for (AActor* Attachment : PlayerAttachments)
		{
			if (Attachment != this)
			{
				CaptureComponent->HiddenActors.AddUnique(Attachment);
			}
		}
	}
	PlayerPawn->OnDestroyed.AddDynamic(this, &AMixedRealityCaptureActor::OnTargetDestroyed);

	if (bSuccess)
	{
		AutoTargeter.Reset();
	}
	return bSuccess;
}

void AMixedRealityCaptureActor::SetAutoBroadcast(const bool bNewValue)
{
	if (bAutoBroadcast != bNewValue)
	{
		if (HasActorBegunPlay())
		{
			if (!bNewValue)
			{
				BroadcastManager.EndCasting();
			}
			else
			{
				BroadcastManager.BeginCasting(CaptureComponent->TextureTarget);
			}
		}
		bAutoBroadcast = bNewValue;
	}
}

bool AMixedRealityCaptureActor::IsBroadcasting()
{
	return HasActorBegunPlay() && BroadcastManager.IsCasting();
}

UTexture* AMixedRealityCaptureActor::GetCaptureTexture()
{
	return CaptureComponent ? CaptureComponent->TextureTarget : nullptr;
}

void AMixedRealityCaptureActor::BeginPlay()
{
	if (bAutoAttachToVRPlayer || bAutoHidePlayer)
	{
		const bool bPlayerFound = MixedRealityCaptureActor_Impl::AssignTargetPlayer(this, bAutoAttachToVRPlayer);
		if (!bPlayerFound)
		{
			AutoTargeter = MakeShareable(new FMRCaptureAutoTargeter(this, bAutoAttachToVRPlayer));
		}
	}

	if (bAutoBroadcast)
	{
		BroadcastManager.BeginCasting(CaptureComponent->TextureTarget);
	}

	Super::BeginPlay();

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	bool bVisualizeCam = false;
	if (GConfig->GetBool(TEXT("/Script/MixedRealityCaptureFramework.MixedRealityCaptureActor"), TEXT("bVisualizeCam"), bVisualizeCam, GEngineIni) && bVisualizeCam)
	{
		UStaticMeshComponent* CamVizualizer = NewObject<UStaticMeshComponent>(this, NAME_None, RF_Transactional | RF_TextExportTransient);
		CamVizualizer->SetupAttachment(CaptureComponent);
		CamVizualizer->SetStaticMesh(DebugVisualizerMesh);
		CamVizualizer->SetRelativeTransform(FTransform(FRotator(90.f, 0.f, 0.f), FVector(7.5, 0.f, 0.f), FVector(0.15f)));
		CamVizualizer->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
		CamVizualizer->CastShadow = false;
		CamVizualizer->RegisterComponent();
	}
#endif
}

void AMixedRealityCaptureActor::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	Super::EndPlay(EndPlayReason);
	BroadcastManager.EndCasting();
}

void AMixedRealityCaptureActor::OnTargetDestroyed(AActor* DestroyedActor)
{
	USceneComponent* CurrentAttachment = RootComponent->GetAttachParent();
	if (CurrentAttachment && CurrentAttachment->GetOwner() == DestroyedActor)
	{
		DetachFromActor(FDetachmentTransformRules::KeepWorldTransform);
		
		if (bAutoAttachToVRPlayer)
		{
			AutoTargeter = MakeShareable(new FMRCaptureAutoTargeter(this, bAutoAttachToVRPlayer));
		}
	}
}

/* AMixedRealityCaptureActor::FCastingModeRestore
 *****************************************************************************/

namespace CastingModeRestore_Impl
{
	ISpectatorScreenController* GetSpectatorScreenController()
	{
		IHeadMountedDisplay* HMD = GEngine->XRSystem.IsValid() ? GEngine->XRSystem->GetHMDDevice() : nullptr;
		if (HMD)
		{
			return HMD->GetSpectatorScreenController();
		}
		return nullptr;
	}
}

AMixedRealityCaptureActor::FCastingModeRestore::FCastingModeRestore()
	: RestoreTexture(nullptr)
	, RestoreMode(ESpectatorScreenMode::SingleEyeCroppedToFill)
	, bIsCasting(false)
{}

bool AMixedRealityCaptureActor::FCastingModeRestore::BeginCasting(UTexture* DisplayTexture)
{
	EndCasting();

	ISpectatorScreenController* const Controller = CastingModeRestore_Impl::GetSpectatorScreenController();
	if (Controller)
	{
		RestoreTexture = Controller->GetSpectatorScreenTexture();
		Controller->SetSpectatorScreenTexture(DisplayTexture);

		RestoreMode = Controller->GetSpectatorScreenMode();
		Controller->SetSpectatorScreenMode(ESpectatorScreenMode::Texture);

		bIsCasting = true;
	}

	return bIsCasting;
}

void AMixedRealityCaptureActor::FCastingModeRestore::EndCasting()
{
	// @TODO: not perfect, if someone external (say Blueprints) overwrote the spectator screen, then this may be undesired
	if (bIsCasting)
	{
		ISpectatorScreenController* const Controller = CastingModeRestore_Impl::GetSpectatorScreenController();
		if (Controller)
		{
			Controller->SetSpectatorScreenMode(RestoreMode);
			Controller->SetSpectatorScreenTexture(RestoreTexture);
		}
		bIsCasting = false;
	}
}

#undef LOCTEXT_NAMESPACE
