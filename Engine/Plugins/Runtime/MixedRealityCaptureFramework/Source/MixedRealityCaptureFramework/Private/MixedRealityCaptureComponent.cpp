// Copyright Epic Games, Inc. All Rights Reserved.

#include "MixedRealityCaptureComponent.h"
#include "Camera/CameraTypes.h"
#include "Engine/Texture2D.h"
#include "IMediaCaptureSupport.h"
#include "MotionControllerComponent.h"
#include "MrcProjectionBillboard.h"
#include "Engine/TextureRenderTarget2D.h"
#include "MediaPlayer.h"
#include "Materials/Material.h"
#include "UObject/ConstructorHelpers.h"
#include "MrcUtilLibrary.h"
#include "Kismet/GameplayStatics.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "MediaCaptureSupport.h"
#include "MrcGarbageMatteCaptureComponent.h"
#include "Engine/StaticMesh.h"
#include "MrcLatencyViewExtension.h"
#include "Misc/ConfigCacheIni.h"
#include "MrcFrameworkSettings.h"
#include "UObject/Package.h"
#include "XRTrackingSystemBase.h" // for OnXRTrackingOriginChanged
#include "PlatformFeatures.h"
#include "SaveGameSystem.h"
#include "Serialization/ObjectAndNameAsStringProxyArchive.h"
#include "SceneRenderBuilderInterface.h"

#if WITH_EDITORONLY_DATA
#include "Components/StaticMeshComponent.h"
#include "Engine/CollisionProfile.h"
#endif

DEFINE_LOG_CATEGORY(LogMixedRealityCapture);

/* FMulticastCVarCommand
 *****************************************************************************/

DECLARE_MULTICAST_DELEGATE(FOnCommandValueChanged);

template<typename T, typename U=T>
class TMulticastCVarCommand : private TAutoConsoleVariable<U>, public FOnCommandValueChanged
{
public:
	TMulticastCVarCommand(const TCHAR* Name, const T& DefaultVal, const TCHAR* Help)
		: TAutoConsoleVariable<U>(Name, (U)DefaultVal, Help)
	{
		TAutoConsoleVariable<U>::AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate::CreateRaw(this, &TMulticastCVarCommand<T,U>::OnChanged));
	}

	void OnChanged(IConsoleVariable* /*This*/)
	{
		Broadcast();
	}

	T GetValue() const
	{
		return (T)TAutoConsoleVariable<U>::GetValueOnGameThread();
	}

	operator T() const
	{
		return GetValue();
	}
};

// specialization to avoid warning C4800 ('int32': forcing value to bool 'true' or 'false')
template<>
bool TMulticastCVarCommand<bool, int32>::GetValue() const
{
	return GetValueOnGameThread() != 0;
}

typedef TMulticastCVarCommand<bool, int32> FMulticastBoolCVar;
typedef TMulticastCVarCommand<float> FMulticastFloatCVar;
typedef TMulticastCVarCommand<int32> FMulticastIntCVar;

/* MRCaptureComponent_Impl
 *****************************************************************************/

namespace MRCaptureComponent_Impl
{
	static FMulticastBoolCVar UseUndistortion(
		TEXT("mrc.undistortion"),
		true,
		TEXT("Enables/Disables the undistortion pass for MixedRealityCaptures. When disabled, the default (black) texture is used instead for the distortion displacement map.")
	);

	static FMulticastBoolCVar UseFocalLenAspect(
		TEXT("mrc.undistortion.bUseFocalAspectRatio"),
		true,
		TEXT("When enabled, to account for stretching from the OpenCV undistortion process, MixedRealityCaptures will scale their projected aspect ratio by the estimated focal length ratio (as reported by OpenCV).")
	);

	static FMulticastFloatCVar DistortionCroppingAmount(
		TEXT("mrc.undistortion.CroppingAmount"),
		0.0f,
		TEXT("A value meant to range from 0 to 1. At one, as part of the undistortion process, OpenCV will attempt to crop out all empty pixels resulting from the process (essentially zooming the image). Zero means no cropping will occur.")
	);

	static FMulticastBoolCVar UseUndistortedFOV(
		TEXT("mrc.undistortion.bUseUndistortedFOV"),
		true,
		TEXT("When enabled, MixedRealityCaptures (MRCs) will use the estimated FOV from the OpenCV undistortion process instead of the FOV the MRC was calibrated with. This accounts for any cropping, etc. done by OpenCV.")
	);

	static FMulticastFloatCVar CaptureFOVOverride(
		TEXT("mrc.FovOverride"),
		0.0f,
		TEXT("When set to be greater than zero, MixedRealityCaptures will use this for the FOV instead of what was previously set.")
	);

	static FMulticastIntCVar TrackingLatencyOverride(
		TEXT("mrc.TrackingLatencyOverride"),
		0,
		TEXT("When set to be greater than zero, MixedRealityCaptures will use this for their TrackingLatency instead of what's set. The higher the value (in ms), the more delay there will be introduced to tracked components.")
	);

	static const FName DefaultDistortionMapParamName(TEXT("DistortionDisplacementMap"));
	static const EHMDTrackingOrigin::Type InvalidOriginType = (EHMDTrackingOrigin::Type)0xFF;

	/**
	 *
	 */
	static void RemoveAllCVarBindings(FDelegateUserObjectConst BoundObject);

	/**
	 *
	 */
	static UMrcGarbageMatteCaptureComponent* CreateGarbageMatteComponent(UMixedRealityCaptureComponent* Outer, USceneComponent* TrackingOrigin);

	/**
	 *
	 */
	template <class T>
	static T* CreateTrackingOriginIntermediaryComponent(UMixedRealityCaptureComponent* Mrc, FName Name);

	/**
	 *
	 */
	static void DestroyIntermediaryAttachParent(UMixedRealityCaptureComponent* Mrc);

	/**
	 *
	 */
	static bool ApplyVideoProcessiongParams(UMaterialInterface* VideoProcessingMat, const FMrcVideoProcessingParams& VidProcessingParams);

	/**
	 *
	 */
	static bool ApplyDistortionMapToMaterial(UMaterialInterface* VideoProcessingMat, UTexture* DistortionDisplacementMap);

	/**
	 *
	 */
	template <class T>
	static T* LoadCalibrationData(const FString& SlotName, const int32 UserIndex);
}

//------------------------------------------------------------------------------
static void MRCaptureComponent_Impl::RemoveAllCVarBindings(FDelegateUserObjectConst BoundObject)
{
	UseUndistortion.RemoveAll(BoundObject);
	UseFocalLenAspect.RemoveAll(BoundObject);
	DistortionCroppingAmount.RemoveAll(BoundObject);
	UseUndistortedFOV.RemoveAll(BoundObject);
	CaptureFOVOverride.RemoveAll(BoundObject);
}

//------------------------------------------------------------------------------
static UMrcGarbageMatteCaptureComponent* MRCaptureComponent_Impl::CreateGarbageMatteComponent(UMixedRealityCaptureComponent* Outer, USceneComponent* TrackingOrigin)
{
	ensureMsgf(Outer->IsActive(), TEXT("Spawning garbage mattes for a MR capture that isn't active."));

	UMrcGarbageMatteCaptureComponent* NewGarbageMatteComp = NewObject<UMrcGarbageMatteCaptureComponent>(Outer, TEXT("MRC_GarbageMatteCapture"), RF_Transient | RF_TextExportTransient);
	NewGarbageMatteComp->CaptureSortPriority = Outer->CaptureSortPriority + 1;
	NewGarbageMatteComp->SetupAttachment(Outer);
	NewGarbageMatteComp->RegisterComponent();

	NewGarbageMatteComp->SetTrackingOrigin(TrackingOrigin);
	
	return NewGarbageMatteComp;
}

//------------------------------------------------------------------------------
template <class T>
static T* MRCaptureComponent_Impl::CreateTrackingOriginIntermediaryComponent(UMixedRealityCaptureComponent* Mrc, FName Name)
{
	T* NewComponent = NewObject<T>(Mrc, Name, RF_Transient | RF_TextExportTransient);

	AActor* Owner = Mrc->GetOwner();
	USceneComponent* HMDRoot = UMrcUtilLibrary::FindAssociatedHMDRoot(Owner);
	if (HMDRoot && HMDRoot->GetOwner() == Owner)
	{
		NewComponent->SetupAttachment(HMDRoot);
	}
	else if (USceneComponent* Parent = Mrc->GetAttachParent())
	{
		NewComponent->SetupAttachment(Parent, Mrc->GetAttachSocketName());
	}
	else
	{
		Owner->SetRootComponent(NewComponent);
	}

	NewComponent->RegisterComponent();
	// for MotionControllerComponents, if this is registered during initialization it will fail to auto-activate and won't track; so force it on here
	NewComponent->Activate(/*bReset =*/false);

	return NewComponent;
}

//------------------------------------------------------------------------------
static void MRCaptureComponent_Impl::DestroyIntermediaryAttachParent(UMixedRealityCaptureComponent* Mrc)
{
	AActor* Owner = Mrc->GetOwner();

	USceneComponent* AttachParent = Mrc->GetAttachParent();
	if (ensure(AttachParent))
	{
		Mrc->DetachFromComponent(FDetachmentTransformRules::KeepRelativeTransform);
		if (USceneComponent* NewParent = AttachParent->GetAttachParent())
		{
			Mrc->AttachToComponent(NewParent, FAttachmentTransformRules::KeepRelativeTransform);
		}

		if (Owner->GetRootComponent() == AttachParent)
		{
			Owner->SetRootComponent(Mrc);
		}

		AttachParent->DestroyComponent();
	}
}

//------------------------------------------------------------------------------
static bool MRCaptureComponent_Impl::ApplyVideoProcessiongParams(UMaterialInterface* VideoProcessingMat, const FMrcVideoProcessingParams& VidProcessingParams)
{
	if (UMaterialInstanceDynamic* VideoProcessingMID = Cast<UMaterialInstanceDynamic>(VideoProcessingMat))
	{
		for (const auto& ScalarParam : VidProcessingParams.MaterialScalarParams)
		{
			VideoProcessingMID->SetScalarParameterValue(ScalarParam.Key, ScalarParam.Value);
		}

		for (const auto& VectorParam : VidProcessingParams.MaterialVectorParams)
		{
			VideoProcessingMID->SetVectorParameterValue(VectorParam.Key, VectorParam.Value);
		}
		return true;
	}
	return false;
}

//------------------------------------------------------------------------------
static bool MRCaptureComponent_Impl::ApplyDistortionMapToMaterial(UMaterialInterface* VideoProcessingMat, UTexture* DistortionDisplacementMap)
{
	if (UMaterialInstanceDynamic* VideoProcessingMID = Cast<UMaterialInstanceDynamic>(VideoProcessingMat))
	{
		FString DistortionMapParamName;
		if (GConfig->GetString(TEXT("/Script/MixedRealityCaptureFramework.MixedRealityFrameworkSettings"), TEXT("DistortionMapParamName"), DistortionMapParamName, GEngineIni))
		{
			VideoProcessingMID->SetTextureParameterValue(*DistortionMapParamName, DistortionDisplacementMap);
		}
		else
		{
			VideoProcessingMID->SetTextureParameterValue(DefaultDistortionMapParamName, DistortionDisplacementMap);
		}
		return true;
	}
	return false;
}

//------------------------------------------------------------------------------
template <class T>
static T* MRCaptureComponent_Impl::LoadCalibrationData(const FString& SlotName, const int32 UserIndex)
{
	T* DataObject = nullptr;

	ISaveGameSystem* SaveSystem = IPlatformFeaturesModule::Get().GetSaveGameSystem();
	if (SaveSystem && (SlotName.Len() > 0) && SaveSystem->DoesSaveGameExist(*SlotName, UserIndex))
	{
		DataObject = Cast<T>(UGameplayStatics::LoadGameFromSlot(*SlotName, UserIndex));
		// since we know the save exists, presume that this failed because the save class is either unknown or mismatched
		if (DataObject == nullptr)
		{
			// fallback to loading the raw data ourselves, assume that whatever class was it was a T subclass
			TArray<uint8> ObjectBytes;
			bool bSuccess = SaveSystem->LoadGame(/*bAttemptToUseUI =*/false, *SlotName, UserIndex, ObjectBytes);
			if (bSuccess && ObjectBytes.Num() > 0)
			{
				// jump to the object tagged serialization portion of the data
				FMemoryReader SaveReader = UGameplayStatics::StripSaveGameHeader(ObjectBytes);

				// attempt some plain raw tagged serialization to try and get the data we care about
				DataObject = NewObject<T>(GetTransientPackage(), T::StaticClass());
				if (DataObject)
				{
					FObjectAndNameAsStringProxyArchive Ar(SaveReader, /*bInLoadIfFindFails =*/true);
					DataObject->Serialize(Ar);
				}
			}
		}
	}
	return DataObject;
}

/* UMixedRealityCaptureComponent
 *****************************************************************************/

//------------------------------------------------------------------------------
UMixedRealityCaptureComponent::UMixedRealityCaptureComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bAutoLoadConfiguration(true)
	, bProjectionDepthTracking(true)
	// initialize to a "unknown" state - we don't know what this was calibrated at
	, RelativeOriginType(MRCaptureComponent_Impl::InvalidOriginType)
	, CalibratedFOV(0.f)
{
	const UMrcFrameworkSettings* MrcSettings = GetDefault<UMrcFrameworkSettings>();

	struct FConstructorStatics
	{
		ConstructorHelpers::FObjectFinder<UMediaPlayer> DefaultMediaSource;
		ConstructorHelpers::FObjectFinder<UMaterial>    DefaultVideoProcessingMaterial;
		ConstructorHelpers::FObjectFinder<UTextureRenderTarget2D> DefaultRenderTarget;
		ConstructorHelpers::FObjectFinder<UTexture2D>   DefaultDistortionDisplacementMap;
#if WITH_EDITORONLY_DATA
		ConstructorHelpers::FObjectFinder<UStaticMesh>  EditorCameraMesh;
#endif

		FConstructorStatics(const UMrcFrameworkSettings* InMrcSettings)
			: DefaultMediaSource(*InMrcSettings->DefaulVideoSource.ToString())
			, DefaultVideoProcessingMaterial(*InMrcSettings->DefaultVideoProcessingMat.ToString())
			, DefaultRenderTarget(*InMrcSettings->DefaultRenderTarget.ToString())
			, DefaultDistortionDisplacementMap(*InMrcSettings->DefaultDistortionDisplacementMap.ToString())
#if WITH_EDITORONLY_DATA
			, EditorCameraMesh(TEXT("/Engine/EditorMeshes/MatineeCam_SM"))
#endif
		{}
	};
	static FConstructorStatics ConstructorStatics(MrcSettings);

	MediaSource = ConstructorStatics.DefaultMediaSource.Object;
	VideoProcessingMaterial = ConstructorStatics.DefaultVideoProcessingMaterial.Object;
	TextureTarget = ConstructorStatics.DefaultRenderTarget.Object;
	DistortionDisplacementMap = ConstructorStatics.DefaultDistortionDisplacementMap.Object;

#if WITH_EDITORONLY_DATA
	if (!IsRunningCommandlet())
	{
		ProxyMesh = ConstructorStatics.EditorCameraMesh.Object;
	}
#endif

	// The default camera-processing (chroma keying) materials assume we're rendering with post-processing (they invert tonemapping, etc.).
	// Also, the spectator screen's back buffer expects the texture data to be in sRGB space (a conversion that happens in post-processing).
	CaptureSource = ESceneCaptureSource::SCS_FinalColorLDR;

	// ensure InitializeComponent() gets called
	bWantsInitializeComponent = true;
}

//------------------------------------------------------------------------------
void UMixedRealityCaptureComponent::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
#if WITH_EDITORONLY_DATA
	UMixedRealityCaptureComponent* This = CastChecked<UMixedRealityCaptureComponent>(InThis);
	Collector.AddReferencedObject(This->ProxyMeshComponent);
#endif

	Super::AddReferencedObjects(InThis, Collector);
}
 
//------------------------------------------------------------------------------
void UMixedRealityCaptureComponent::OnRegister()
{
	Super::OnRegister();

#if WITH_EDITORONLY_DATA
	AActor* MyOwner = GetOwner();
	if (MyOwner)
	{
		if (ProxyMeshComponent == nullptr)
		{
			ProxyMeshComponent = NewObject<UStaticMeshComponent>(MyOwner, NAME_None, RF_Transactional | RF_TextExportTransient);
			ProxyMeshComponent->SetupAttachment(this);
			ProxyMeshComponent->bIsEditorOnly = true;
			ProxyMeshComponent->SetStaticMesh(ProxyMesh);
			ProxyMeshComponent->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
			ProxyMeshComponent->bHiddenInGame = true;
			ProxyMeshComponent->CastShadow    = false;
			ProxyMeshComponent->CreationMethod = CreationMethod;
			ProxyMeshComponent->RegisterComponent();
		}
	}
#endif
}

//------------------------------------------------------------------------------
void UMixedRealityCaptureComponent::Activate(bool bReset)
{
	Super::Activate(bReset);

	if (IsActive())
	{
		RefreshDevicePairing();
		RefreshTrackingOriginOffset();

		if (!ProjectionActor)
		{
			ProjectionActor = NewObject<UChildActorComponent>(this, TEXT("MRC_ProjectionPlane"), RF_Transient | RF_TextExportTransient);
			ProjectionActor->SetChildActorClass(AMrcProjectionActor::StaticClass());
			ProjectionActor->SetupAttachment(this);

			ProjectionActor->RegisterComponent();

			AMrcProjectionActor* ProjectionActorObj = CastChecked<AMrcProjectionActor>(ProjectionActor->GetChildActor());
			ProjectionActorObj->SetProjectionMaterial(VideoProcessingMaterial);
			ProjectionActorObj->SetProjectionAspectRatio(GetDesiredAspectRatio());

			if (ensure(ProjectionActorObj->ProjectionComponent))
			{
				ProjectionActorObj->ProjectionComponent->DepthOffset = ProjectionDepthOffset;
				ProjectionActorObj->ProjectionComponent->EnableHMDDepthTracking(bProjectionDepthTracking);
			}
		}

		RefreshCameraFeed();

		MRCaptureComponent_Impl::CaptureFOVOverride.AddUObject(this, &UMixedRealityCaptureComponent::RefreshFOV);
		MRCaptureComponent_Impl::UseUndistortedFOV.AddUObject(this, &UMixedRealityCaptureComponent::RefreshFOV);
		MRCaptureComponent_Impl::DistortionCroppingAmount.AddUObject(this, &UMixedRealityCaptureComponent::RefreshDistortionDisplacementMap);
		MRCaptureComponent_Impl::UseFocalLenAspect.AddUObject(this, &UMixedRealityCaptureComponent::RefreshProjectionDimensions);
		MRCaptureComponent_Impl::UseUndistortion.AddUObject(this, &UMixedRealityCaptureComponent::RefreshDistortionDisplacementMap);

		FXRTrackingSystemDelegates::OnXRTrackingOriginChanged.AddUObject(this, &UMixedRealityCaptureComponent::OnTrackingOriginChanged);
	}	
}

//------------------------------------------------------------------------------
void UMixedRealityCaptureComponent::Deactivate()
{
	Super::Deactivate();

	if (!IsActive())
	{
		FXRTrackingSystemDelegates::OnXRTrackingOriginChanged.RemoveAll(this);
		MRCaptureComponent_Impl::RemoveAllCVarBindings(this);

		if (MediaSource)
		{
			MediaSource->Close();
		}

		// the GarbageMatte component's lifetime is governed by ApplyCalibrationData

		if (ProjectionActor)
		{
			ProjectionActor->DestroyComponent();
			ProjectionActor = nullptr;
		}

		if (PairedTracker || TrackingOriginOffset)
		{
			MRCaptureComponent_Impl::DestroyIntermediaryAttachParent(this);
			PairedTracker = nullptr;
			TrackingOriginOffset = nullptr;
		}
	}
}

//------------------------------------------------------------------------------
void UMixedRealityCaptureComponent::InitializeComponent()
{
	Super::InitializeComponent();

	if (!VideoProcessingMaterial->IsA<UMaterialInstanceDynamic>())
	{
		SetVidProjectionMat(UMaterialInstanceDynamic::Create(VideoProcessingMaterial, this));
	}

	const UWorld* MyWorld = GetWorld();
	if (MyWorld && MyWorld->IsGameWorld() && bAutoLoadConfiguration)
	{
		LoadDefaultConfiguration();
	}

	RefreshFOV();
	RefreshCameraFeed();
}

//------------------------------------------------------------------------------
void UMixedRealityCaptureComponent::OnUpdateTransform(EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport)
{
#if WITH_EDITORONLY_DATA
	if (ProxyMeshComponent)
	{
		const FTransform WorldXform = GetComponentToWorld();
		ProxyMeshComponent->SetWorldTransform(WorldXform);
	}
#endif

	Super::OnUpdateTransform(UpdateTransformFlags, Teleport);
}

//------------------------------------------------------------------------------
void UMixedRealityCaptureComponent::OnComponentDestroyed(bool bDestroyingHierarchy)
{
#if WITH_EDITORONLY_DATA
	if (ProxyMeshComponent)
	{
		ProxyMeshComponent->DestroyComponent();
	}
#endif 
	FXRTrackingSystemDelegates::OnXRTrackingOriginChanged.RemoveAll(this);
	MRCaptureComponent_Impl::RemoveAllCVarBindings(this);

	if (ProjectionActor)
	{
		ProjectionActor->DestroyComponent();
	}

	if (PairedTracker)
	{
		PairedTracker->DestroyComponent();
	}

	if (TrackingOriginOffset)
	{
		TrackingOriginOffset->DestroyComponent();
	}

	if (GarbageMatteCaptureComponent)
	{
		GarbageMatteCaptureComponent->ShowOnlyActors.Empty();
		GarbageMatteCaptureComponent->DestroyComponent();
	}

	Super::OnComponentDestroyed(bDestroyingHierarchy);
}

//------------------------------------------------------------------------------
#if WITH_EDITOR
bool UMixedRealityCaptureComponent::GetEditorPreviewInfo(float /*DeltaTime*/, FMinimalViewInfo& ViewOut)
{
	ViewOut.Location = GetComponentLocation();
	ViewOut.Rotation = GetComponentRotation();

	ViewOut.FOV = FOVAngle;

	ViewOut.AspectRatio = GetDesiredAspectRatio();
	ViewOut.bConstrainAspectRatio = true;

	// see default in FSceneViewInitOptions
	ViewOut.bUseFieldOfViewForLOD = true;
	
 	ViewOut.ProjectionMode = ProjectionType;
	ViewOut.OrthoWidth = OrthoWidth;

	// see BuildProjectionMatrix() in SceneCaptureRendering.cpp
	ViewOut.OrthoNearClipPlane = 0.0f;
	ViewOut.OrthoFarClipPlane  = UE_FLOAT_HUGE_DISTANCE / 4.0f;

	ViewOut.PostProcessBlendWeight = PostProcessBlendWeight;
	if (PostProcessBlendWeight > 0.0f)
	{
		ViewOut.PostProcessSettings = PostProcessSettings;
	}

	return true;
}
#endif	// WITH_EDITOR

//------------------------------------------------------------------------------
const AActor* UMixedRealityCaptureComponent::GetViewOwner() const
{
	return GetProjectionActor();
}

//------------------------------------------------------------------------------
void UMixedRealityCaptureComponent::UpdateSceneCaptureContents(FSceneInterface* Scene, ISceneRenderBuilder& SceneRenderBuilder)
{ 
	if (TextureTarget)
	{
		TextureTarget->TargetGamma = GEngine ? GEngine->GetDisplayGamma() : 2.2f;
	}

	if (!ViewExtension.IsValid())
	{
		ViewExtension = FSceneViewExtensions::NewExtension<FMrcLatencyViewExtension>(this);
		FMotionDelayService::RegisterDelayClient(ViewExtension.ToSharedRef());
	}

	bool* bPreCommandQueuePtr = SceneRenderBuilder.GetAllocator().Malloc<bool>();

	SceneRenderBuilder.AddCommand([bPreCommandQueuePtr, ViewExtension = ViewExtension, Scene]
	{
		*bPreCommandQueuePtr = ViewExtension->SetupPreCapture(Scene);
	});

	Super::UpdateSceneCaptureContents(Scene, SceneRenderBuilder);

	SceneRenderBuilder.AddCommand([bPreCommandQueuePtr, ViewExtension = ViewExtension, Scene]
	{
		if (*bPreCommandQueuePtr)
		{
			ViewExtension->SetupPostCapture(Scene);
		}
	});
}

//------------------------------------------------------------------------------
void UMixedRealityCaptureComponent::RefreshCameraFeed()
{
	const UWorld* MyWorld = GetWorld();
	if (CaptureFeedRef.DeviceURL.IsEmpty() && IsActive() && HasBeenInitialized() && MyWorld && MyWorld->IsGameWorld())
	{
		TArray<FMediaCaptureDeviceInfo> CaptureDevices;
		MediaCaptureSupport::EnumerateVideoCaptureDevices(CaptureDevices);

		if (CaptureDevices.Num() > 0)
		{
			FMRCaptureFeedDelegate::FDelegate OnOpenCallback;
			OnOpenCallback.BindUFunction(this, GET_FUNCTION_NAME_CHECKED(UMixedRealityCaptureComponent, OnVideoFeedOpened));

			UAsyncTask_OpenMrcVidCaptureDevice::OpenMrcVideoCaptureDevice(CaptureDevices[0], MediaSource, OnOpenCallback);
		}
	}
	else
	{
		SetCaptureDevice(CaptureFeedRef);
	}
}

//------------------------------------------------------------------------------
void UMixedRealityCaptureComponent::RefreshDevicePairing()
{
#if WITH_EDITORONLY_DATA
	AActor* MyOwner = GetOwner();
	if (MyOwner && MyOwner->GetWorld() && MyOwner->GetWorld()->IsGameWorld())
#endif
	{
		if (!TrackingSourceName.IsNone())
		{
			USceneComponent* Parent = GetAttachParent();
			UMotionControllerComponent* PreDefinedTracker = Cast<UMotionControllerComponent>(Parent);
			const bool bNeedsInternalController = (!PreDefinedTracker || PreDefinedTracker->MotionSource != TrackingSourceName);

			if (bNeedsInternalController)
			{
				if (!PairedTracker)
				{
					PairedTracker = MRCaptureComponent_Impl::CreateTrackingOriginIntermediaryComponent<UMotionControllerComponent>(this, TEXT("MRC_PairedTracker"));
					AttachToComponent(PairedTracker, FAttachmentTransformRules::KeepRelativeTransform);
				}

				PairedTracker->MotionSource = TrackingSourceName;
			}			
		}
		else if (PairedTracker)
		{
			if (ensure(PairedTracker == GetAttachParent()))
			{
				MRCaptureComponent_Impl::DestroyIntermediaryAttachParent(this);
			}
			else
			{
				PairedTracker->DestroyComponent(/*bPromoteChildren =*/true);
			}
			PairedTracker = nullptr;

			RefreshTrackingOriginOffset();
		}
	}
}

void UMixedRealityCaptureComponent::RefreshTrackingOriginOffset()
{
#if WITH_EDITORONLY_DATA
	AActor* MyOwner = GetOwner();
	if (MyOwner && MyOwner->GetWorld() && MyOwner->GetWorld()->IsGameWorld())
#endif
	{
		if (GEngine && GEngine->XRSystem.IsValid())
		{
			EHMDTrackingOrigin::Type ActiveOriginType = GEngine->XRSystem->GetTrackingOrigin();

			const bool bNeedsTrackingOriginOffset = (!PairedTracker || GarbageMatteCaptureComponent) &&
				(ActiveOriginType != RelativeOriginType) && (RelativeOriginType != MRCaptureComponent_Impl::InvalidOriginType);

			if (bNeedsTrackingOriginOffset)
			{
				if (!TrackingOriginOffset)
				{
					TrackingOriginOffset = MRCaptureComponent_Impl::CreateTrackingOriginIntermediaryComponent<USceneComponent>(this, TEXT("MRC_TrackingOriginOffset"));
				}

				FTransform FloorToEyeTransform = FTransform::Identity;
				IHeadMountedDisplay* Hmd = GEngine->XRSystem->GetHMDDevice();
				const bool bHasEyeTransform = GEngine->XRSystem->GetFloorToEyeTrackingTransform(FloorToEyeTransform);

				if (bHasEyeTransform)
				{
					switch (RelativeOriginType)
					{
					case EHMDTrackingOrigin::LocalFloor:
						{
							TrackingOriginOffset->SetRelativeTransform(FloorToEyeTransform);
							break;
						}

					case EHMDTrackingOrigin::Local:
						{
							TrackingOriginOffset->SetRelativeTransform(FloorToEyeTransform.Inverse());
							break;
						}
					}
				}

				if (!PairedTracker)
				{
					AttachToComponent(TrackingOriginOffset, FAttachmentTransformRules::KeepRelativeTransform);
				}
				if (GarbageMatteCaptureComponent)
				{
					GarbageMatteCaptureComponent->SetTrackingOrigin(TrackingOriginOffset);
				}
			}
			else if (TrackingOriginOffset)
			{
				if (TrackingOriginOffset == GetAttachParent())
				{
					MRCaptureComponent_Impl::DestroyIntermediaryAttachParent(this);
				}
				else
				{
					TrackingOriginOffset->DestroyComponent(/*bPromoteChildren =*/true);
				}

				if (GarbageMatteCaptureComponent)
				{
					USceneComponent* GarbageMatteOrigin = PairedTracker ? PairedTracker->GetAttachParent() : GetAttachParent();
					GarbageMatteCaptureComponent->SetTrackingOrigin(GarbageMatteOrigin);
				}

				TrackingOriginOffset = nullptr;
			}
		}
	}
}

//------------------------------------------------------------------------------
void UMixedRealityCaptureComponent::SetVidProjectionMat(UMaterialInterface* NewMaterial)
{
	bool bResetParams = MRCaptureComponent_Impl::ApplyVideoProcessiongParams(NewMaterial, VideoProcessingParams);
	bResetParams &= MRCaptureComponent_Impl::ApplyDistortionMapToMaterial(NewMaterial, DistortionDisplacementMap);

	if (!bResetParams)
	{
		// should we convert it to be a MID?
	}

	VideoProcessingMaterial = NewMaterial;
	if (AMrcProjectionActor* ProjectionTarget = GetProjectionActor())
	{
		ProjectionTarget->SetProjectionMaterial(NewMaterial);
	}
}

//------------------------------------------------------------------------------
void UMixedRealityCaptureComponent::SetVidProcessingParams(const FMrcVideoProcessingParams& NewVidProcessingParams)
{
	MRCaptureComponent_Impl::ApplyVideoProcessiongParams(VideoProcessingMaterial, NewVidProcessingParams);
	VideoProcessingParams = NewVidProcessingParams;
}

//------------------------------------------------------------------------------
void UMixedRealityCaptureComponent::SetDeviceAttachment(FName SourceName)
{
	TrackingSourceName = SourceName;
	RefreshDevicePairing();
}

//------------------------------------------------------------------------------
void UMixedRealityCaptureComponent::DetatchFromDevice()
{
	TrackingSourceName = NAME_None;
	RefreshDevicePairing();
}

//------------------------------------------------------------------------------
bool UMixedRealityCaptureComponent::IsTracked() const
{
	return PairedTracker && PairedTracker->IsTracked();
}

//------------------------------------------------------------------------------
void UMixedRealityCaptureComponent::SetCaptureDevice(const FMrcVideoCaptureFeedIndex& FeedRef)
{
	const UWorld* MyWorld = GetWorld();
	if (HasBeenInitialized() && IsActive() && MyWorld && MyWorld->IsGameWorld())
	{
		if (MediaSource)
		{
			if (!FeedRef.IsSet(MediaSource))
			{
				FMRCaptureFeedDelegate::FDelegate OnOpenCallback;
				OnOpenCallback.BindUFunction(this, GET_FUNCTION_NAME_CHECKED(UMixedRealityCaptureComponent, OnVideoFeedOpened));

				UAsyncTask_OpenMrcVidCaptureFeed::OpenMrcVideoCaptureFeed(FeedRef, MediaSource, OnOpenCallback);
			}
			else
			{
				CaptureFeedRef = FeedRef;
				RefreshProjectionDimensions();
			}
		}
	}
	else
	{
		CaptureFeedRef = FeedRef;
	}
}

//------------------------------------------------------------------------------
void UMixedRealityCaptureComponent::SetLensDistortionParameters(const FOpenCVLensDistortionParameters& ModelRef)
{
	if (ModelRef != LensDistortionParameters)
	{
		LensDistortionParameters = ModelRef;
		RefreshDistortionDisplacementMap();
	}
}

//------------------------------------------------------------------------------
int32 UMixedRealityCaptureComponent::GetTrackingDelay() const
{
	return (MRCaptureComponent_Impl::TrackingLatencyOverride > 0) ? (int32)MRCaptureComponent_Impl::TrackingLatencyOverride : TrackingLatency;
}

//------------------------------------------------------------------------------
void UMixedRealityCaptureComponent::SetTrackingDelay(int32 DelayMS)
{
	TrackingLatency = FMath::Max(DelayMS, 0);
}

//------------------------------------------------------------------------------
void UMixedRealityCaptureComponent::SetProjectionDepthOffset(float DepthOffset)
{
	ProjectionDepthOffset = DepthOffset;

	AMrcProjectionActor* ProjActor = GetProjectionActor();
	if (ProjActor && ProjActor->ProjectionComponent)
	{
		ProjActor->ProjectionComponent->DepthOffset = ProjectionDepthOffset;
	}
}

//------------------------------------------------------------------------------
AActor* UMixedRealityCaptureComponent::GetProjectionActor_K2() const
{
	return GetProjectionActor();
}

//------------------------------------------------------------------------------
AMrcProjectionActor* UMixedRealityCaptureComponent::GetProjectionActor() const
{
	return ProjectionActor ? Cast<AMrcProjectionActor>(ProjectionActor->GetChildActor()) : nullptr;
}

//------------------------------------------------------------------------------
void UMixedRealityCaptureComponent::SetEnableProjectionDepthTracking(bool bEnable)
{
	bProjectionDepthTracking = bEnable;

	AMrcProjectionActor* ProjActor = GetProjectionActor();
	if (ProjActor && ProjActor->ProjectionComponent)
	{
		ProjActor->ProjectionComponent->EnableHMDDepthTracking(bEnable);
	}
}

//------------------------------------------------------------------------------
float UMixedRealityCaptureComponent::GetDesiredAspectRatio() const
{
	float DesiredAspectRatio = 0.0f;

	if (MediaSource)
	{
		const int32 SelectedTrack = MediaSource->GetSelectedTrack(EMediaPlayerTrack::Video);
		DesiredAspectRatio = MediaSource->GetVideoTrackAspectRatio(SelectedTrack, MediaSource->GetTrackFormat(EMediaPlayerTrack::Video, SelectedTrack));
	}

	if (DesiredAspectRatio == 0.0f)
	{
		if (TextureTarget)
		{
			DesiredAspectRatio = TextureTarget->GetSurfaceWidth() / TextureTarget->GetSurfaceHeight();
		}
		else
		{
			DesiredAspectRatio = 16.f / 9.f;
		}
	}

	if (MRCaptureComponent_Impl::UseUndistortion && MRCaptureComponent_Impl::UseFocalLenAspect && !LensDistortionParameters.IsIdentity() && UndistortedCameraInfo.FocalLengthRatio > 0)
	{ 
		DesiredAspectRatio *= UndistortedCameraInfo.FocalLengthRatio;
	}

	return DesiredAspectRatio;
}

//------------------------------------------------------------------------------
void UMixedRealityCaptureComponent::RefreshDistortionDisplacementMap()
{
	if (MRCaptureComponent_Impl::UseUndistortion && !LensDistortionParameters.IsIdentity() && TextureTarget)
	{
		DistortionDisplacementMap = LensDistortionParameters.CreateUndistortUVDisplacementMap(FIntPoint(TextureTarget->SizeX, TextureTarget->SizeY), MRCaptureComponent_Impl::DistortionCroppingAmount, UndistortedCameraInfo);
	}
	else
	{
		const UMrcFrameworkSettings* DefaultSettings = GetDefault<UMrcFrameworkSettings>();
		DistortionDisplacementMap = Cast<UTexture2D>(DefaultSettings->DefaultDistortionDisplacementMap.TryLoad());
	}
	MRCaptureComponent_Impl::ApplyDistortionMapToMaterial(VideoProcessingMaterial, DistortionDisplacementMap);

	if (MRCaptureComponent_Impl::UseFocalLenAspect)
	{
		RefreshProjectionDimensions();
	}
	if (MRCaptureComponent_Impl::UseUndistortedFOV)
	{
		RefreshFOV();
	}
}

//------------------------------------------------------------------------------
void UMixedRealityCaptureComponent::RefreshFOV()
{
	if (MRCaptureComponent_Impl::CaptureFOVOverride > 0.0f)
	{
		FOVAngle = MRCaptureComponent_Impl::CaptureFOVOverride;
	}
	else if (MRCaptureComponent_Impl::UseUndistortion && MRCaptureComponent_Impl::UseUndistortedFOV && !LensDistortionParameters.IsIdentity() && UndistortedCameraInfo.HorizontalFOV > 0)
	{
		FOVAngle = UndistortedCameraInfo.HorizontalFOV;
	}
	else if (CalibratedFOV > 0)
	{
		FOVAngle = CalibratedFOV;
	}
}

//------------------------------------------------------------------------------
void UMixedRealityCaptureComponent::OnTrackingOriginChanged(const IXRTrackingSystem* /*TrackingSys*/)
{
	RefreshTrackingOriginOffset();
}

//------------------------------------------------------------------------------
void UMixedRealityCaptureComponent::OnVideoFeedOpened(const FMrcVideoCaptureFeedIndex& FeedRef)
{
	CaptureFeedRef = FeedRef;
	RefreshProjectionDimensions();

	OnCaptureSourceOpened.Broadcast(FeedRef);
}

//------------------------------------------------------------------------------
void UMixedRealityCaptureComponent::RefreshProjectionDimensions()
{
	if (AMrcProjectionActor* VidProjection = GetProjectionActor())
	{
		VidProjection->SetProjectionAspectRatio( GetDesiredAspectRatio() );
	}
}

//------------------------------------------------------------------------------
bool UMixedRealityCaptureComponent::SaveAsDefaultConfiguration_K2()
{
	return SaveAsDefaultConfiguration();
}

//------------------------------------------------------------------------------
bool UMixedRealityCaptureComponent::SaveAsDefaultConfiguration() const
{
	FString EmptySlotName;
	return SaveConfiguration(EmptySlotName, INDEX_NONE);
}

//------------------------------------------------------------------------------
bool UMixedRealityCaptureComponent::SaveConfiguration_K2(const FString& SlotName, int32 UserIndex)
{
	return SaveConfiguration(SlotName, UserIndex);
}

//------------------------------------------------------------------------------
bool UMixedRealityCaptureComponent::SaveConfiguration(const FString& SlotName, int32 UserIndex) const
{
	UMrcCalibrationData* SaveGameInstance = ConstructCalibrationData();

	const UMrcCalibrationSaveGame* DefaultSaveData = GetDefault<UMrcCalibrationSaveGame>();
	check(DefaultSaveData);
	const FString& LocalSlotName  = SlotName.Len() > 0 ? SlotName  : DefaultSaveData->SaveSlotName;
	const uint32   LocalUserIndex = SlotName.Len() > 0 ? UserIndex : DefaultSaveData->UserIndex;

	const bool bSuccess = UGameplayStatics::SaveGameToSlot(SaveGameInstance, LocalSlotName, LocalUserIndex);
	if (bSuccess)
	{
		UE_LOGF(LogMixedRealityCapture, Log, "UMixedRealityCaptureComponent::SaveConfiguration to slot %ls user %i Succeeded.", *LocalSlotName, LocalUserIndex);
	} 
	else
	{
		UE_LOGF(LogMixedRealityCapture, Warning, "UMixedRealityCaptureComponent::SaveConfiguration to slot %ls user %i Failed!", *LocalSlotName, LocalUserIndex);
	}
	return bSuccess;
}

//------------------------------------------------------------------------------
bool UMixedRealityCaptureComponent::LoadDefaultConfiguration()
{
	FString EmptySlotName;
	return LoadConfiguration(EmptySlotName, INDEX_NONE);
}

//------------------------------------------------------------------------------
bool UMixedRealityCaptureComponent::LoadConfiguration(const FString& SlotName, int32 UserIndex)
{
	const UMrcCalibrationSaveGame* DefaultSaveData = GetDefault<UMrcCalibrationSaveGame>();
	check(DefaultSaveData);
	const FString& LocalSlotName  = SlotName.Len() > 0 ? SlotName  : DefaultSaveData->SaveSlotName;
	const uint32   LocalUserIndex = SlotName.Len() > 0 ? UserIndex : DefaultSaveData->UserIndex;

	UMrcCalibrationData* SaveGameInstance = MRCaptureComponent_Impl::LoadCalibrationData<UMrcCalibrationData>(LocalSlotName, LocalUserIndex);
	if (SaveGameInstance == nullptr)
	{
		UE_LOGF(LogMixedRealityCapture, Warning, "UMixedRealityCaptureComponent::LoadConfiguration from slot %ls user %i Failed!", *LocalSlotName, LocalUserIndex);
		return false;
	}

	ApplyCalibrationData(SaveGameInstance);

	UE_LOGF(LogMixedRealityCapture, Log, "UMixedRealityCaptureComponent::LoadConfiguration from slot %ls user %i Succeeded.", *LocalSlotName, LocalUserIndex);
	return true;
}

//------------------------------------------------------------------------------
UMrcCalibrationData* UMixedRealityCaptureComponent::ConstructCalibrationData_Implementation() const
{
	UMrcCalibrationData* ConfigData = NewObject<UMrcCalibrationData>(GetTransientPackage());
	FillOutCalibrationData(ConfigData);

	return ConfigData;
}

//------------------------------------------------------------------------------
void UMixedRealityCaptureComponent::FillOutCalibrationData(UMrcCalibrationData* Dst) const
{
	if (Dst)
	{
		// view info
		{
			Dst->LensData.FOV = FOVAngle;
			Dst->LensData.DistortionParameters = LensDistortionParameters;
		}
		// alignment info
		{
			const FTransform RelativeXform = GetRelativeTransform();
			Dst->AlignmentData.CameraOrigin = RelativeXform.GetLocation();
			Dst->AlignmentData.Orientation = RelativeXform.GetRotation().Rotator();

			Dst->AlignmentData.TrackingAttachmentId = TrackingSourceName;
			
			if (RelativeOriginType == MRCaptureComponent_Impl::InvalidOriginType)
			{
				Dst->AlignmentData.TrackingOrigin = (GEngine && GEngine->XRSystem.IsValid()) ? GEngine->XRSystem->GetTrackingOrigin() : EHMDTrackingOrigin::LocalFloor;
			}
			else
			{
				Dst->AlignmentData.TrackingOrigin = RelativeOriginType;
			}
		}
		// compositing info
		{
			Dst->CompositingData.CaptureDeviceURL = CaptureFeedRef;
			Dst->CompositingData.DepthOffset = ProjectionDepthOffset;
			Dst->CompositingData.TrackingLatency = GetTrackingDelay();
			Dst->CompositingData.VideoProcessingParams = VideoProcessingParams;
		}
		// garbage matte
		{
			if (GarbageMatteCaptureComponent)
			{
				GarbageMatteCaptureComponent->GetGarbageMatteData(Dst->GarbageMatteSaveDatas);
			}
			else
			{
				Dst->GarbageMatteSaveDatas.Empty();
			}
		}
	}
}


//------------------------------------------------------------------------------
void UMixedRealityCaptureComponent::ApplyCalibrationData_Implementation(UMrcCalibrationData* ConfigData)
{
	if (ConfigData)
	{
		// view data
		{
			CalibratedFOV = ConfigData->LensData.FOV;
			SetLensDistortionParameters(ConfigData->LensData.DistortionParameters);

			RefreshFOV();
		}
		// alignment data
		{
			SetDeviceAttachment(ConfigData->AlignmentData.TrackingAttachmentId);

			SetRelativeLocation(ConfigData->AlignmentData.CameraOrigin);
			SetRelativeRotation(ConfigData->AlignmentData.Orientation);

			RelativeOriginType = ConfigData->AlignmentData.TrackingOrigin;
		}
		// compositing data
		{
			SetCaptureDevice(ConfigData->CompositingData.CaptureDeviceURL);
			SetTrackingDelay(ConfigData->CompositingData.TrackingLatency);
			SetProjectionDepthOffset(ConfigData->CompositingData.DepthOffset);
			SetVidProcessingParams(ConfigData->CompositingData.VideoProcessingParams);
		}
		// garbage matte
		{
			if (ConfigData->GarbageMatteSaveDatas.Num() > 0)
			{
				if (GarbageMatteCaptureComponent == nullptr)
				{
					USceneComponent* GarbageMatteOrigin = TrackingOriginOffset ? ToRawPtr(TrackingOriginOffset) : (PairedTracker ? PairedTracker->GetAttachParent() : GetAttachParent());
					GarbageMatteCaptureComponent = MRCaptureComponent_Impl::CreateGarbageMatteComponent(this, GarbageMatteOrigin);
				}
				GarbageMatteCaptureComponent->ApplyCalibrationData(ConfigData);
			}
			else if (GarbageMatteCaptureComponent)
			{
				GarbageMatteCaptureComponent->DestroyComponent();
				GarbageMatteCaptureComponent = nullptr;
			}
		}

		// needs to happen at the end, because there are factors above used to determine if we need an offset component
		RefreshTrackingOriginOffset();
	}
}

//------------------------------------------------------------------------------
bool UMixedRealityCaptureComponent::SetGarbageMatteActor(AMrcGarbageMatteActor* Actor)
{
	bool bSuccess = false;
	if (GarbageMatteCaptureComponent)
	{
		GarbageMatteCaptureComponent->SetGarbageMatteActor(Actor);
		bSuccess = true;
	}
	else if (IsActive())
	{
		USceneComponent* GarbageMatteOrigin = TrackingOriginOffset ? ToRawPtr(TrackingOriginOffset) : (PairedTracker ? PairedTracker->GetAttachParent() : GetAttachParent());
		GarbageMatteCaptureComponent = MRCaptureComponent_Impl::CreateGarbageMatteComponent(this, GarbageMatteOrigin);
		GarbageMatteCaptureComponent->SetGarbageMatteActor(Actor);

		bSuccess = true;
	}

	return bSuccess;
}
