// Copyright Epic Games, Inc. All Rights Reserved.

#include "MrcCalibrationData.h"

UMrcCalibrationData::UMrcCalibrationData(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{}

UMrcCalibrationSaveGame::UMrcCalibrationSaveGame(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SaveSlotName = TEXT("MrcCalibration");
	UserIndex = 0;
	ConfigurationSaveVersion = 1;
}
