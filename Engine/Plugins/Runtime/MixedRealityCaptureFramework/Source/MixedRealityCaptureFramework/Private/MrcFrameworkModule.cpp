// Copyright Epic Games, Inc. All Rights Reserved.

#include "IMrcFrameworkModule.h"
#include "Modules/ModuleManager.h" // for IMPLEMENT_MODULE()
#include "Kismet/GameplayStatics.h" // for DoesSaveGameExist()
#include "MixedRealityCaptureActor.h"
#include "MixedRealityCaptureComponent.h"
#include "UObject/UObjectIterator.h"
#include "MotionDelayBuffer.h" // for SetEnabled()

#if WITH_EDITOR
#include "Editor.h" // for FEditorDelegates::PostPIEStarted
#endif

class FMrcFrameworkModule : public IMrcFrameworkModule
{
public:
	FMrcFrameworkModule();

public:
	//~ IModuleInterface interface

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	// IMrcFrameworkModule
	virtual class AMixedRealityCaptureActor* GetMixedRealityCaptureActor() override { return MixedRealityCaptureActor.Get(); }

private:
	void OnWorldCreated(UWorld* NewWorld);
#if WITH_EDITOR
	void OnPieWorldCreated(bool bIsSimulating);
#endif

private:
	FDelegateHandle WorldEventBinding;
	FString TargetConfigName;
	int32 TargetConfigIndex;
	TWeakObjectPtr<AMixedRealityCaptureActor> MixedRealityCaptureActor;
};

FMrcFrameworkModule::FMrcFrameworkModule()
	: TargetConfigIndex(0)
{}

void FMrcFrameworkModule::StartupModule()
{
	const UMrcCalibrationSaveGame* DefaultSaveData = GetDefault<UMrcCalibrationSaveGame>();
	TargetConfigName  = DefaultSaveData->SaveSlotName;
	TargetConfigIndex = DefaultSaveData->UserIndex;

	{
		WorldEventBinding = GEngine->OnWorldAdded().AddRaw(this, &FMrcFrameworkModule::OnWorldCreated);
		FCoreUObjectDelegates::PostLoadMapWithWorld.AddRaw(this, &FMrcFrameworkModule::OnWorldCreated);

#if WITH_EDITOR
		FEditorDelegates::PostPIEStarted.AddRaw(this, &FMrcFrameworkModule::OnPieWorldCreated);
#endif
	}

	FMotionDelayService::SetEnabled(true);
}

void FMrcFrameworkModule::ShutdownModule()
{
#if WITH_EDITOR
	FEditorDelegates::PostPIEStarted.RemoveAll(this);
#endif

	FCoreUObjectDelegates::PostLoadMapWithWorld.RemoveAll(this);
	if (GEngine)
	{
		GEngine->OnWorldAdded().Remove(WorldEventBinding);
	}
}

void FMrcFrameworkModule::OnWorldCreated(UWorld* NewWorld)
{
#if WITH_EDITORONLY_DATA
	const bool bIsGameInst = !IsRunningCommandlet() && NewWorld && NewWorld->IsGameWorld();
	if (bIsGameInst)
#endif 
	{
		const bool bHasMRConfigFile = UGameplayStatics::DoesSaveGameExist(TargetConfigName, TargetConfigIndex);
		if (bHasMRConfigFile)
		{
			UMixedRealityCaptureComponent* MRCaptureComponent = nullptr;
			for (TObjectIterator<UMixedRealityCaptureComponent> ObjIt; ObjIt; ++ObjIt)
			{
				if (ObjIt->GetWorld() == NewWorld)
				{
					MRCaptureComponent = *ObjIt;
				}
			}

			if (MRCaptureComponent == nullptr)
			{
				AMixedRealityCaptureActor* MRActor = NewWorld->SpawnActor<AMixedRealityCaptureActor>();
				MRCaptureComponent = MRActor->CaptureComponent;
				MixedRealityCaptureActor = MRActor;
			}

			MRCaptureComponent->LoadConfiguration(TargetConfigName, TargetConfigIndex);
		}
	}
}

#if WITH_EDITOR
void FMrcFrameworkModule::OnPieWorldCreated(bool bIsSimulating)
{
	UWorld* PieWorld = GEditor->GetPIEWorldContext()->World();
	if (!bIsSimulating && PieWorld)
	{
		OnWorldCreated(PieWorld);
	}
}
#endif // WITH_EDITOR

IMPLEMENT_MODULE(FMrcFrameworkModule, MixedRealityCaptureFramework);
