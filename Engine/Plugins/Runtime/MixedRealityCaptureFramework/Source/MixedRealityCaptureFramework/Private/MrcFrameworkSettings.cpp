// Copyright Epic Games, Inc. All Rights Reserved.

#include "MrcFrameworkSettings.h"

UMrcFrameworkSettings::UMrcFrameworkSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, DefaulVideoSource(TEXT("/MixedRealityCaptureFramework/MrcVideoSource"))
	, DefaultVideoProcessingMat(TEXT("/MixedRealityCaptureFramework/M_MrcVideoProcessing"))
	, DefaultRenderTarget(TEXT("/MixedRealityCaptureFramework/T_MrcRenderTarget"))
	, DefaultDistortionDisplacementMap(TEXT("/Engine/EngineResources/Black"))
	, DefaulGarbageMatteMesh(TEXT("/MixedRealityCaptureFramework/SM_GarbageMattePlane"))
	, DefaulGarbageMatteMaterial(TEXT("/MixedRealityCaptureFramework/M_GarbageMatte"))
	, DefaulGarbageMatteTarget(TEXT("/MixedRealityCaptureFramework/T_MrcGarbageMatteRenderTarget"))
{}
