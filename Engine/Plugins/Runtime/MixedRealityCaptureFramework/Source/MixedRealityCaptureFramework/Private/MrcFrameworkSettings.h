// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MrcFrameworkSettings.generated.h"

UCLASS(config=Engine)
class UMrcFrameworkSettings : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY(Config)
	FSoftObjectPath DefaulVideoSource;

	UPROPERTY(Config)
	FSoftObjectPath DefaultVideoProcessingMat;

	UPROPERTY(Config)
	FSoftObjectPath DefaultRenderTarget;

	UPROPERTY(Config)
	FSoftObjectPath DefaultDistortionDisplacementMap;

	UPROPERTY(Config)
	FSoftObjectPath DefaulGarbageMatteMesh;

	UPROPERTY(Config)
	FSoftObjectPath DefaulGarbageMatteMaterial;

	UPROPERTY(Config)
	FSoftObjectPath DefaulGarbageMatteTarget;
};
