// Copyright Epic Games, Inc. All Rights Reserved.

#include "MrcGarbageMatteCaptureComponent.h"

#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/World.h"
#include "IMrcFocalDriver.h"
#include "Materials/Material.h"
#include "MrcCalibrationData.h"
#include "MrcFrameworkSettings.h"
#include "UObject/ConstructorHelpers.h"

/* UMrcGarbageMatteCaptureComponent
 *****************************************************************************/

//------------------------------------------------------------------------------
UMrcGarbageMatteCaptureComponent::UMrcGarbageMatteCaptureComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCaptureEveryFrame = true;
	PrimitiveRenderMode = ESceneCapturePrimitiveRenderMode::PRM_UseShowOnlyList;
	CaptureSource = ESceneCaptureSource::SCS_SceneColorHDR;
	PostProcessBlendWeight = 0.0f;
	ShowFlags.SetAtmosphere(false);
	ShowFlags.SetFog(false);

	const UMrcFrameworkSettings* MrcSettings = GetDefault<UMrcFrameworkSettings>();
	ConstructorHelpers::FObjectFinder<UTextureRenderTarget2D> DefaultTarget(*MrcSettings->DefaulGarbageMatteTarget.ToString());
	TextureTarget = DefaultTarget.Object;

	GarbageMatteActorClass = AMrcGarbageMatteActor::StaticClass();
}

//------------------------------------------------------------------------------
void UMrcGarbageMatteCaptureComponent::OnComponentDestroyed(bool bDestroyingHierarchy)
{
	CleanupSpawnedActors();
	GarbageMatteActor = nullptr;

	Super::OnComponentDestroyed(bDestroyingHierarchy);
}

//-----------------------------------------------------------------------------
void UMrcGarbageMatteCaptureComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	PollFocalDriver();
}

//------------------------------------------------------------------------------
const AActor* UMrcGarbageMatteCaptureComponent::GetViewOwner() const
{
	// This lets SetOnlyOwnerSee on the garbage matte actor make it visible only to this capture component.
	// Basically the "owner" Actor's pointer is being used as an ID for who renders the actor.
	return GarbageMatteActor;
}

//------------------------------------------------------------------------------
void UMrcGarbageMatteCaptureComponent::SetTrackingOrigin(USceneComponent* InTrackingOrigin)
{
	TrackingOriginPtr = InTrackingOrigin;
	
	if (GarbageMatteActor)
	{
		GarbageMatteActor->AttachToComponent(InTrackingOrigin, FAttachmentTransformRules::SnapToTargetNotIncludingScale);
		// have to reset the relative offset manually in case the actor was already attached to this component (result from a component destroy)
		GarbageMatteActor->SetActorRelativeTransform(FTransform::Identity);
	}
}

//------------------------------------------------------------------------------
void UMrcGarbageMatteCaptureComponent::ApplyCalibrationData_Implementation(const UMrcCalibrationData* ConfigData)
{
	if (ConfigData)
	{
		if (GarbageMatteActor == nullptr)
		{
			SetGarbageMatteActor( SpawnNewGarbageMatteActor(TrackingOriginPtr.Get()) );
		}

		FOVAngle = ConfigData->LensData.FOV;

		if (ensure(GarbageMatteActor))
		{
			GarbageMatteActor->ApplyCalibrationData(ConfigData->GarbageMatteSaveDatas);
		}
	}
}

//------------------------------------------------------------------------------
void UMrcGarbageMatteCaptureComponent::GetGarbageMatteData(TArray<FMrcGarbageMatteSaveData>& GarbageMatteDataOut)
{
	if (GarbageMatteActor)
	{
		GarbageMatteActor->GetGarbageMatteData(GarbageMatteDataOut);
	}
	else
	{
		GarbageMatteDataOut.Empty();
	}
}

//-----------------------------------------------------------------------------
void UMrcGarbageMatteCaptureComponent::SetFocalDriver(TScriptInterface<IMrcFocalDriver> InFocalDriver)
{
	FocalDriver = InFocalDriver;
}

//------------------------------------------------------------------------------
void UMrcGarbageMatteCaptureComponent::CleanupSpawnedActors()
{
	while(SpawnedActors.Num() > 0)
	{
		AMrcGarbageMatteActor* Actor = SpawnedActors[0];
		if (Actor)
		{
			Actor->Destroy();
		}
		SpawnedActors.RemoveAtSwap(0);
	}
}

//------------------------------------------------------------------------------
void UMrcGarbageMatteCaptureComponent::PollFocalDriver()
{
	if (FocalDriver.GetObject() != nullptr && FocalDriver.GetObject()->Implements<UMrcFocalDriver>())
	{
		FOVAngle = IMrcFocalDriver::Execute_GetHorizontalFieldOfView(FocalDriver.GetObject());
	}
}

//------------------------------------------------------------------------------
AMrcGarbageMatteActor* UMrcGarbageMatteCaptureComponent::SpawnNewGarbageMatteActor_Implementation(USceneComponent* InTrackingOrigin)
{
	AMrcGarbageMatteActor* NewGarbageMatteActor = nullptr;

	UWorld* MyWorld = GetWorld();
#if WITH_EDITORONLY_DATA
	if (MyWorld && MyWorld->IsGameWorld())
#endif
	{
		FActorSpawnParameters SpawnParameters;
		SpawnParameters.Name = TEXT("MR_GarbageMatteActor");

		UClass* SpawnClass = GarbageMatteActorClass;
		if (SpawnClass == nullptr)
		{
			SpawnClass = AMrcGarbageMatteActor::StaticClass();
		}

		AActor* SpawnedActor = MyWorld->SpawnActor(GarbageMatteActorClass, /*Location =*/nullptr, /*Rotation =*/nullptr, SpawnParameters);
		if (ensure(SpawnedActor) && InTrackingOrigin) 
		{
			SpawnedActor->AttachToComponent(InTrackingOrigin, FAttachmentTransformRules::SnapToTargetNotIncludingScale);
		}
		NewGarbageMatteActor = CastChecked<AMrcGarbageMatteActor>(SpawnedActor, ECastCheckedType::NullAllowed);
		SpawnedActors.Add(NewGarbageMatteActor);
	}
	
	return NewGarbageMatteActor;
}

//------------------------------------------------------------------------------
void UMrcGarbageMatteCaptureComponent::SetGarbageMatteActor(AMrcGarbageMatteActor* NewActor)
{
	TArray<FMrcGarbageMatteSaveData> GarbageMatteData;
	if (GarbageMatteActor)
	{
		GarbageMatteActor->GetGarbageMatteData(GarbageMatteData);

		ShowOnlyActors.Remove(GarbageMatteActor);

		//Verify if the actual GarbageMatteActor has been spawned by us and destroy if its the case.
		const int32 FoundIndex = SpawnedActors.Find(GarbageMatteActor);
		if (FoundIndex != INDEX_NONE)
		{
			SpawnedActors.RemoveAtSwap(FoundIndex);
			GarbageMatteActor->Destroy();
			GarbageMatteActor = nullptr;
		}
	}

	GarbageMatteActor = NewActor;

	if (NewActor)
	{
		ShowOnlyActors.Add(NewActor);
		NewActor->ApplyCalibrationData(GarbageMatteData);

		if (TrackingOriginPtr.IsValid())
		{
			NewActor->AttachToComponent(TrackingOriginPtr.Get(), FAttachmentTransformRules::SnapToTargetNotIncludingScale);
		}
	}
}

/* AMrcGarbageMatteActor
 *****************************************************************************/

//------------------------------------------------------------------------------
AMrcGarbageMatteActor::AMrcGarbageMatteActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	const UMrcFrameworkSettings* MrcSettings = GetDefault<UMrcFrameworkSettings>();

	struct FConstructorStatics
	{
		ConstructorHelpers::FObjectFinder<UStaticMesh> DefaultGarbageMatteMesh;
		ConstructorHelpers::FObjectFinder<UMaterial>   DefaultGarbageMatteMaterial;

		FConstructorStatics(const UMrcFrameworkSettings* InMrcSettings)
			: DefaultGarbageMatteMesh(*InMrcSettings->DefaulGarbageMatteMesh.ToString())
			, DefaultGarbageMatteMaterial(*InMrcSettings->DefaulGarbageMatteMaterial.ToString())
		{}
	};
	static FConstructorStatics ConstructorStatics(MrcSettings);

	GarbageMatteMesh = ConstructorStatics.DefaultGarbageMatteMesh.Object;
	GarbageMatteMaterial = ConstructorStatics.DefaultGarbageMatteMaterial.Object;

	RootComponent = CreateDefaultSubobject<USceneComponent>(TEXT("TrackingOriginPt"));
}

//------------------------------------------------------------------------------
void AMrcGarbageMatteActor::ApplyCalibrationData(const TArray<FMrcGarbageMatteSaveData>& GarbageMatteData)
{
	for (UPrimitiveComponent* OldGarbageMatte : GarbageMattes)
	{
		OldGarbageMatte->DestroyComponent();
	}
	GarbageMattes.Empty(GarbageMatteData.Num());

	for (const FMrcGarbageMatteSaveData& Data : GarbageMatteData)
	{
		AddNewGabageMatte(Data);
	}
}

//------------------------------------------------------------------------------
UPrimitiveComponent* AMrcGarbageMatteActor::AddNewGabageMatte(const FMrcGarbageMatteSaveData& GarbageMatteData)
{
	UPrimitiveComponent* NewMatte = CreateGarbageMatte(GarbageMatteData);
	GarbageMattes.Add(NewMatte);

	return NewMatte;
}

//------------------------------------------------------------------------------
UPrimitiveComponent* AMrcGarbageMatteActor::CreateGarbageMatte_Implementation(const FMrcGarbageMatteSaveData& GarbageMatteData)
{
	UStaticMeshComponent* MeshComponent = NewObject<UStaticMeshComponent>(this, NAME_None);
	MeshComponent->SetStaticMesh(GarbageMatteMesh);
	MeshComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);
	MeshComponent->SetCastShadow(false);
	MeshComponent->SetRelativeTransform(GarbageMatteData.Transform);
	MeshComponent->SetMaterial(0, GarbageMatteMaterial);
	MeshComponent->SetOnlyOwnerSee(true);
	MeshComponent->SetMobility(EComponentMobility::Movable);
	MeshComponent->SetupAttachment(RootComponent);
	MeshComponent->RegisterComponent();

	return MeshComponent;
}

//------------------------------------------------------------------------------
void AMrcGarbageMatteActor::GetGarbageMatteData(TArray<FMrcGarbageMatteSaveData>& GarbageMatteDataOut)
{
	GarbageMatteDataOut.Empty(GarbageMattes.Num());

	FMrcGarbageMatteSaveData GarbageMatteData;
	for (const UPrimitiveComponent* GarbageMatte : GarbageMattes)
	{
		GarbageMatteData.Transform = GarbageMatte->GetRelativeTransform();
		GarbageMatteDataOut.Add(GarbageMatteData);
	}
}
