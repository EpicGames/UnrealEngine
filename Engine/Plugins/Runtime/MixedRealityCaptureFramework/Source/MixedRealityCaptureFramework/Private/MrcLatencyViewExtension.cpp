// Copyright Epic Games, Inc. All Rights Reserved.

#include "MrcLatencyViewExtension.h"
#include "MixedRealityCaptureComponent.h"
#include "RenderingThread.h" // for ENQUEUE_RENDER_COMMAND
#include "MotionControllerComponent.h"

namespace MRLatencyViewExtension_Impl
{
	TAutoConsoleVariable<int32> CVarMotionCaptureLatencyOverride(
		TEXT("mr.MotionCaptureLatencyOverride"),
		0,
		TEXT("When set, will track historical motion data, using it to simulate latency when rendering to the MR capture view (helpful when trying to sync with a video feed).\n")
		TEXT("     0: don't use the override - default to the MR capture's calibrated delay (default)\n")
		TEXT(" [1:n]: use motion controller transforms from n milliseconds ago when rendering the MR capture"),
		ECVF_Default);

	static uint32 GetDesiredDelay(TWeakObjectPtr<UMixedRealityCaptureComponent> Target);
	static UMotionControllerComponent* GetPairedTracker(TWeakObjectPtr<UMixedRealityCaptureComponent> Target);
}

//------------------------------------------------------------------------------
static uint32 MRLatencyViewExtension_Impl::GetDesiredDelay(TWeakObjectPtr<UMixedRealityCaptureComponent> Target)
{
	uint32 DesiredDelay = CVarMotionCaptureLatencyOverride.GetValueOnGameThread();
	if (DesiredDelay <= 0 && Target.IsValid())
	{
		DesiredDelay = Target->GetTrackingDelay();
	}
	return DesiredDelay;
}

static UMotionControllerComponent* MRLatencyViewExtension_Impl::GetPairedTracker(TWeakObjectPtr<UMixedRealityCaptureComponent> Target)
{
	UMotionControllerComponent* PairedTracker = nullptr;
	if (Target.IsValid())
	{
		PairedTracker = Cast<UMotionControllerComponent>(Target->GetAttachParent());
	}
	return PairedTracker;
}

/* FMrcLatencyViewExtension
 *****************************************************************************/

//------------------------------------------------------------------------------
FMrcLatencyViewExtension::FMrcLatencyViewExtension(const FAutoRegister& AutoRegister, UMixedRealityCaptureComponent* InOwner)
	: FMotionDelayClient(AutoRegister)
	, Owner(InOwner)
	, CachedRenderDelay(0)
{}

//------------------------------------------------------------------------------
bool FMrcLatencyViewExtension::SetupPreCapture(FSceneInterface* Scene)
{
	const bool bSimulateLatency = (CachedRenderDelay > 0);
	if (bSimulateLatency)
	{
		if (Owner.IsValid())
		{
			CachedOwnerTransform = Owner->GetComponentToWorld();
			if (UMotionControllerComponent* PairedTracker = MRLatencyViewExtension_Impl::GetPairedTracker(Owner))
			{
				FTransform OriginTransform = FTransform::Identity;
				if (USceneComponent* VROrigin = PairedTracker->GetAttachParent())
				{
					OriginTransform = VROrigin->GetComponentToWorld();
				}

				FTransform DelayTransform;
				if (FindDelayTransform(PairedTracker, CachedRenderDelay, DelayTransform))
				{
					const FTransform RelativeTransform = Owner->GetRelativeTransform();

					// Replace the parent MotionControllerComponent's transform with a delayed one (to 
					// simulate latency so the video feed better matches up with the virtual camera position)
					//
					// @TODO : this breaks down if any of the transform components are absolute, or 
					//         if something is attached to a socket (see: UpdateComponentToWorldWithParent)
					const FTransform NewComponentToWorldTransform = RelativeTransform * DelayTransform * OriginTransform;

					Owner->SetComponentToWorld(NewComponentToWorldTransform);
				}
			}
		}

		TSharedPtr<FMrcLatencyViewExtension, ESPMode::ThreadSafe> ThisPtr = SharedThis(this);
		ENQUEUE_RENDER_COMMAND(PreMRCaptureCommand)(
			[ThisPtr, Scene](FRHICommandListImmediate& /*RHICmdList*/)
			{
				ThisPtr->Apply_RenderThread(Scene);
			}
		);
	}
	return bSimulateLatency;
}

//------------------------------------------------------------------------------
void FMrcLatencyViewExtension::SetupPostCapture(FSceneInterface* Scene)
{
	const bool bPreCommandEnqueued = (CachedRenderDelay > 0);
	if (bPreCommandEnqueued)
	{
		TSharedPtr<FMrcLatencyViewExtension, ESPMode::ThreadSafe> ThisPtr = SharedThis(this);
		ENQUEUE_RENDER_COMMAND(PostMRCaptureCommand)(
			[ThisPtr, Scene](FRHICommandListImmediate& /*RHICmdList*/)
			{
				ThisPtr->Restore_RenderThread(Scene);
			}
		);

		if (Owner.IsValid())
		{
			Owner->SetComponentToWorld(CachedOwnerTransform);
		}
	}
}

//------------------------------------------------------------------------------
uint32 FMrcLatencyViewExtension::GetDesiredDelay() const
{
	uint32 DesiredDelay = MRLatencyViewExtension_Impl::CVarMotionCaptureLatencyOverride.GetValueOnGameThread();
	if (DesiredDelay <= 0 && Owner.IsValid())
	{
		DesiredDelay = Owner->GetTrackingDelay();
	}
	return DesiredDelay;
}

void FMrcLatencyViewExtension::GetExemptTargets(TArray<USceneComponent*>& ExemptTargets) const
{
// 	if (UMotionControllerComponent* PairedTracker = MRLatencyViewExtension_Impl::GetPairedTracker(Owner))
// 	{
// 		ExemptTargets.Add(PairedTracker);
// 	}
}

//------------------------------------------------------------------------------
void FMrcLatencyViewExtension::BeginRenderViewFamily(FSceneViewFamily& ViewFamily)
{
	CachedRenderDelay = GetDesiredDelay();
	FMotionDelayClient::BeginRenderViewFamily(ViewFamily);
}

//------------------------------------------------------------------------------
bool FMrcLatencyViewExtension::IsActiveThisFrame_Internal(const FSceneViewExtensionContext& Context) const
{
	return Owner.IsValid() && Owner->IsActive() && FMotionDelayClient::IsActiveThisFrame_Internal(Context);
}
