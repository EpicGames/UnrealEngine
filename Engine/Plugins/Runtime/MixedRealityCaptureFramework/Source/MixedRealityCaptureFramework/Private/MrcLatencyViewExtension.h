// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MotionDelayBuffer.h"

class FAutoRegister;


class UMixedRealityCaptureComponent;
class FMotionDelayBuffer;
class FSceneInterface;

class FMrcLatencyViewExtension : public FMotionDelayClient
{
public:
	FMrcLatencyViewExtension(const FAutoRegister& AutoRegister, UMixedRealityCaptureComponent* Owner);
	virtual ~FMrcLatencyViewExtension() {}

	bool SetupPreCapture(FSceneInterface* Scene);
	void SetupPostCapture(FSceneInterface* Scene);

public:
	/** FMotionDelayClient interface */
	virtual uint32 GetDesiredDelay() const override;
	virtual void GetExemptTargets(TArray<USceneComponent*>& ExemptTargets) const override;

public:
	/** ISceneViewExtension interface */
	virtual void BeginRenderViewFamily(FSceneViewFamily& InViewFamily) override;
	virtual bool IsActiveThisFrame_Internal(const FSceneViewExtensionContext& Context) const override;

private:
	TWeakObjectPtr<UMixedRealityCaptureComponent> Owner;
	uint32 CachedRenderDelay = 0;
	FTransform CachedOwnerTransform;
};
