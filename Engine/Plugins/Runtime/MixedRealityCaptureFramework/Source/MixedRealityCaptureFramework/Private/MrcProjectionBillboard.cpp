// Copyright Epic Games, Inc. All Rights Reserved.

#include "MrcProjectionBillboard.h"
#include "UObject/ConstructorHelpers.h"
#include "Materials/Material.h"
#include "Engine/Engine.h"
#include "GameFramework/PlayerController.h"
#include "IXRTrackingSystem.h"
#include "HeadMountedDisplayFunctionLibrary.h" // for GetDeviceWorldPose()
#include "IXRCamera.h"
#include "MrcFrameworkSettings.h" // for DefaultVideoProcessingMat

//------------------------------------------------------------------------------
UMixedRealityCaptureBillboard::UMixedRealityCaptureBillboard(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.bStartWithTickEnabled = false;

	if (AActor* Owner = GetOwner())
	{
		AddTickPrerequisiteActor(Owner);
	}
}

//------------------------------------------------------------------------------
void UMixedRealityCaptureBillboard::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	FVector ViewOffset = FVector::ForwardVector * (GNearClippingPlane + FMath::Max(0.01f, DepthOffset));
	if (GEngine->XRSystem.IsValid())
	{
		IIdentifiableXRDevice* HMDDevice = GEngine->XRSystem->GetXRCamera().Get();
		if (HMDDevice != nullptr)
		{
			FVector  HMDPos;
			FRotator HMDRot;

			bool bHasTracking = false, bHasPositionalTracking = false;
			UHeadMountedDisplayFunctionLibrary::GetDeviceWorldPose(/*WorldContext =*/this, HMDDevice, /*[out]*/bHasTracking, /*[out]*/HMDRot, /*[out]*/bHasPositionalTracking, /*[out]*/HMDPos);

			if (bHasPositionalTracking)
			{
				// ASSUMPTION: this is positioned (attached) directly relative to some view
				USceneComponent* MRViewComponent = GetAttachParent();
				if (ensure(MRViewComponent))
				{
					const FVector ViewWorldPos = MRViewComponent->GetComponentLocation();
					const FVector ViewToHMD = (HMDPos - ViewWorldPos);
					const FVector ViewFacingDir = MRViewComponent->GetForwardVector();

					const float HMDDepth = (/*DotProduct: */ViewFacingDir | ViewToHMD) + DepthOffset;
					if (HMDDepth > GNearClippingPlane)
					{
						ViewOffset = FVector::ForwardVector * HMDDepth;
					}
				}
			}
		}
	}

	SetRelativeLocationAndRotation(ViewOffset, FRotator::ZeroRotator);
}

//------------------------------------------------------------------------------
void UMixedRealityCaptureBillboard::EnableHMDDepthTracking(bool bEnable)
{
	SetComponentTickEnabled(bEnable);
	if (!bEnable)
	{
		SetRelativeLocation( FVector::ForwardVector * (GNearClippingPlane + FMath::Max(0.01f, DepthOffset)) );
	}
}

/* AMrcProjectionActor
 *****************************************************************************/

//------------------------------------------------------------------------------
AMrcProjectionActor::AMrcProjectionActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryActorTick.bCanEverTick = true;
	PrimaryActorTick.bStartWithTickEnabled = true;

	const UMrcFrameworkSettings* MrcSettings = GetDefault<UMrcFrameworkSettings>();

	struct FConstructorStatics
	{
		ConstructorHelpers::FObjectFinder<UMaterial> DefaultMaterial;
		float DefaultAspectRatio;

		FConstructorStatics(const UMrcFrameworkSettings* InMrcSettings)
			: DefaultMaterial(*InMrcSettings->DefaultVideoProcessingMat.ToString())
			, DefaultAspectRatio(16.f / 9.f)
		{}
	};
	static FConstructorStatics ConstructorStatics(MrcSettings);

	RootComponent = CreateDefaultSubobject<USceneComponent>(TEXT("SceneRoot"));

	UWorld* MyWorld = GetWorld();	
	ProjectionComponent = CreateDefaultSubobject<UMixedRealityCaptureBillboard>(TEXT("MRC_ProjectionMesh"));
	ProjectionComponent->SetupAttachment(RootComponent);
	ProjectionComponent->AddElement(ConstructorStatics.DefaultMaterial.Object, 
		/*DistanceToOpacityCurve =*/nullptr, 
		/*bSizeIsInScreenSpace =*/true, 
		/*BaseSizeX =*/1.0f, 
		/*BaseSizeY=*/ConstructorStatics.DefaultAspectRatio, 
		/*DistanceToSizeCurve =*/nullptr);
	ProjectionComponent->CastShadow = false;
	ProjectionComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);
	// expects that this actor (or one of its owners) is used as the scene's view 
	// actor (hidden in the editor via UMixedRealityCaptureBillboard::GetHiddenEditorViews)
	ProjectionComponent->bOnlyOwnerSee = MyWorld && MyWorld->IsGameWorld();
	ProjectionComponent->EnableHMDDepthTracking(true);
	ProjectionComponent->SetRelativeLocation(FVector::ForwardVector * (GNearClippingPlane + 0.01f));
}

//------------------------------------------------------------------------------
void AMrcProjectionActor::BeginPlay()
{
	Super::BeginPlay();

	UWorld* MyWorld = GetWorld();
	for (FConstPlayerControllerIterator PlayerIt = MyWorld->GetPlayerControllerIterator(); PlayerIt; ++PlayerIt)
	{
		if (APlayerController* PlayerController = PlayerIt->Get())
		{
			PlayerController->HiddenPrimitiveComponents.AddUnique(ProjectionComponent);
		}
	}

	if (ProjectionComponent)
	{
		ProjectionComponent->SetRelativeLocation(FVector::ForwardVector * (GNearClippingPlane + 0.01f));
	}
}

//------------------------------------------------------------------------------
void AMrcProjectionActor::Tick(float DeltaSeconds)
{
	Super::Tick(DeltaSeconds);
}

//------------------------------------------------------------------------------
void AMrcProjectionActor::SetProjectionMaterial(UMaterialInterface* VidProcessingMat)
{
	ProjectionComponent->SetMaterial(/*ElementIndex =*/0, VidProcessingMat);
}

//------------------------------------------------------------------------------
void AMrcProjectionActor::SetProjectionAspectRatio(const float NewAspectRatio)
{
	FMaterialSpriteElement& Sprite = ProjectionComponent->Elements[0];
	if (Sprite.BaseSizeY != NewAspectRatio)
	{
		Sprite.BaseSizeY = NewAspectRatio;
		ProjectionComponent->MarkRenderStateDirty();
	}	
}
