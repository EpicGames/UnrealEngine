// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/MaterialBillboardComponent.h"
#include "MrcProjectionBillboard.generated.h"

class APawn;
class UMaterialInstance;

UCLASS()
class UMixedRealityCaptureBillboard : public UMaterialBillboardComponent
{
	GENERATED_UCLASS_BODY()

public:
	//~ UActorComponent interface
	virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;

public:
	//~ UPrimitiveComponent interface
#if WITH_EDITOR
	virtual uint64 GetHiddenEditorViews() const override
	{
		// we don't want this billboard crowding the editor window, so hide it from all editor
		// views (however, we do want to see it in preview windows, which this doesn't affect)
		return 0xFFFFFFFFFFFFFFFF;
	}
#endif // WITH_EDITOR

public:
	float DepthOffset = 0.0f;


	void EnableHMDDepthTracking(bool bEnable = true);
};

/* AMrcProjectionActor
 *****************************************************************************/

UCLASS(notplaceable)
class AMrcProjectionActor : public AActor
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TObjectPtr<UMixedRealityCaptureBillboard> ProjectionComponent;

public:
	//~ AActor interface
	virtual void BeginPlay() override;
	virtual void Tick(float DeltaSeconds) override;

public:
	void SetProjectionMaterial(UMaterialInterface* VidProcessingMat);
	void SetProjectionAspectRatio(const float NewAspectRatio);

private:
	UPROPERTY(Transient)
	TWeakObjectPtr<USceneComponent> AttachTarget;
};
