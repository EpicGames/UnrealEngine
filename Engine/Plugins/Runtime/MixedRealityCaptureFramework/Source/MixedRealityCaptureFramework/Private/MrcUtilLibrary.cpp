// Copyright Epic Games, Inc. All Rights Reserved.

#include "MrcUtilLibrary.h"
#include "IMrcFrameworkModule.h"
#include "Kismet/GameplayStatics.h"
#include "GameFramework/Pawn.h"
#include "Camera/CameraComponent.h"
#include "GameFramework/PlayerController.h"
#include "Engine/Engine.h"
#include "Engine/LocalPlayer.h"
#include "MixedRealityCaptureActor.h"
#include "Modules/ModuleManager.h"

/* MrcUtilLibrary_Impl 
 *****************************************************************************/

namespace MrcUtilLibrary_Impl
{
	static bool IsActorOwnedByPlayer(AActor* ActorInst, ULocalPlayer* Player);
	static AMixedRealityCaptureActor* GetMixedRealityCaptureActor();
}

static bool MrcUtilLibrary_Impl::IsActorOwnedByPlayer(AActor* ActorInst, ULocalPlayer* Player)
{
	bool bIsOwned = false;
	if (ActorInst)
	{
		if (UWorld* ActorWorld = ActorInst->GetWorld())
		{
			APlayerController* Controller = Player->GetPlayerController(ActorWorld);
			if (Controller && ActorInst->IsOwnedBy(Controller))
			{
				bIsOwned = true;
			}
			else if (Controller)
			{
				APawn* PlayerPawn = Controller->GetPawnOrSpectator();
				if (PlayerPawn && ActorInst->IsOwnedBy(PlayerPawn))
				{
					bIsOwned = true;
				}
				else if (USceneComponent* ActorRoot = ActorInst->GetRootComponent())
				{
					USceneComponent* AttachParent = ActorRoot->GetAttachParent();
					if (AttachParent)
					{
						bIsOwned = IsActorOwnedByPlayer(AttachParent->GetOwner(), Player);
					}
				}
			}
		}
	}
	return bIsOwned;
}

static AMixedRealityCaptureActor* MrcUtilLibrary_Impl::GetMixedRealityCaptureActor()
{
	IMrcFrameworkModule* FrameworkModule = static_cast<IMrcFrameworkModule*>(FModuleManager::Get().LoadModule(TEXT("MixedRealityCaptureFramework")));
	return FrameworkModule ? FrameworkModule->GetMixedRealityCaptureActor() : nullptr;
}

/* UMrcUtilLibrary 
 *****************************************************************************/

UMrcUtilLibrary::UMrcUtilLibrary(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool UMrcUtilLibrary::IsMixedRealityCaptureBroadcasting()
{
	AMixedRealityCaptureActor* const CaptureActor = MrcUtilLibrary_Impl::GetMixedRealityCaptureActor();
	return CaptureActor ? CaptureActor->IsBroadcasting() : false;
}

void UMrcUtilLibrary::SetMixedRealityCaptureBroadcasting(bool bEnable)
{
	AMixedRealityCaptureActor* const CaptureActor = MrcUtilLibrary_Impl::GetMixedRealityCaptureActor();
	if (CaptureActor)
	{
		CaptureActor->SetAutoBroadcast(bEnable);
	}
}

UTexture* UMrcUtilLibrary::GetMixedRealityCaptureTexture()
{
	AMixedRealityCaptureActor* const CaptureActor = MrcUtilLibrary_Impl::GetMixedRealityCaptureActor();
	return CaptureActor ? CaptureActor->GetCaptureTexture() : nullptr;
}

APawn* UMrcUtilLibrary::FindAssociatedPlayerPawn(AActor* ActorInst)
{
	APawn* PlayerPawn = nullptr;

	if (UWorld* TargetWorld = ActorInst->GetWorld())
	{
		const TArray<ULocalPlayer*>& LocalPlayers = GEngine->GetGamePlayers(TargetWorld);
		for (ULocalPlayer* Player : LocalPlayers)
		{
			if (MrcUtilLibrary_Impl::IsActorOwnedByPlayer(ActorInst, Player))
			{
				PlayerPawn = Player->GetPlayerController(TargetWorld)->GetPawnOrSpectator();
				break;
			}
		}
	}
	return PlayerPawn;
}

USceneComponent* UMrcUtilLibrary::FindAssociatedHMDRoot(AActor* ActorInst)
{
	APawn* PlayerPawn = FindAssociatedPlayerPawn(ActorInst);
	return UMrcUtilLibrary::GetHMDRootComponent(PlayerPawn);
}

USceneComponent* UMrcUtilLibrary::GetHMDRootComponent(const UObject* WorldContextObject, int32 PlayerIndex)
{
	const APawn* PlayerPawn = UGameplayStatics::GetPlayerPawn(WorldContextObject, PlayerIndex);
	return UMrcUtilLibrary::GetHMDRootComponent(PlayerPawn);
}

USceneComponent* UMrcUtilLibrary::GetHMDRootComponent(const APawn* PlayerPawn)
{
	USceneComponent* VROrigin = nullptr;
	if (UCameraComponent* HMDCamera = UMrcUtilLibrary::GetHMDCameraComponent(PlayerPawn))
	{
		VROrigin = HMDCamera->GetAttachParent();
	}
	return VROrigin;
}

UCameraComponent* UMrcUtilLibrary::GetHMDCameraComponent(const APawn* PlayerPawn)
{
	UCameraComponent* HMDCam = nullptr;
	if (PlayerPawn)
	{
		for (UActorComponent* Component : PlayerPawn->GetComponents())
		{
			if (UCameraComponent* Camera = Cast<UCameraComponent>(Component))
			{
				if (Camera->bLockToHmd)
				{
					HMDCam = Camera;
					break;
				}
				else if (HMDCam == nullptr)
				{
					HMDCam = Camera;
				}
			}
		}
	}
	return HMDCam;
}
