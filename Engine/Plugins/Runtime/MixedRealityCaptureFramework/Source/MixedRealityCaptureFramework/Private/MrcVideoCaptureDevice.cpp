// Copyright Epic Games, Inc. All Rights Reserved.

#include "MrcVideoCaptureDevice.h"
#include "Math/Range.h"
#include "MediaPlayer.h"
#include "MediaCaptureSupport.h" // for EnumerateVideoCaptureDevices()
#include "IMediaCaptureSupport.h" // for FMediaCaptureDeviceInfo
#include "MixedRealityCaptureComponent.h" // for LogMixedRealityCapture
#include "IMediaEventSink.h" // for EMediaEvent
#include "Misc/ConfigCacheIni.h"
#include "Tickable.h"

/* FMrcVideoCaptureFeedIndex
 *****************************************************************************/

FMrcVideoCaptureFeedIndex::FMrcVideoCaptureFeedIndex()
	: StreamIndex(0)
	, FormatIndex(0)
{}

FMrcVideoCaptureFeedIndex::FMrcVideoCaptureFeedIndex::FMrcVideoCaptureFeedIndex(UMediaPlayer* MediaPlayer)
	: StreamIndex(0)
	, FormatIndex(0)
{
	if (MediaPlayer != nullptr)
	{
		DeviceURL   = MediaPlayer->GetUrl();
		StreamIndex = MediaPlayer->GetSelectedTrack(EMediaPlayerTrack::Video);
		FormatIndex = MediaPlayer->GetTrackFormat(EMediaPlayerTrack::Video, StreamIndex);
	}
}

FMrcVideoCaptureFeedIndex::FMrcVideoCaptureFeedIndex(FMediaCaptureDeviceInfo& DeviceInfo)
	: DeviceURL(DeviceInfo.Url)
	, StreamIndex(0)
	, FormatIndex(0)
{}

bool FMrcVideoCaptureFeedIndex::IsSet(UMediaPlayer* MediaPlayer) const
{
	return MediaPlayer && (MediaPlayer->GetSelectedTrack(EMediaPlayerTrack::Video) == StreamIndex) && 
		(MediaPlayer->GetTrackFormat(EMediaPlayerTrack::Video, StreamIndex) == FormatIndex) && MediaPlayer->GetUrl() == DeviceURL;
}

bool FMrcVideoCaptureFeedIndex::IsDeviceUrlValid() const
{
	if (!DeviceURL.IsEmpty())
	{
		TArray<FMediaCaptureDeviceInfo> ActiveDevices;
		MediaCaptureSupport::EnumerateVideoCaptureDevices(ActiveDevices);

		for (const FMediaCaptureDeviceInfo& ConnectedDevice : ActiveDevices)
		{
			if (ConnectedDevice.Url == DeviceURL)
			{
				return true;
			}
		}
	}
	return false;
}

/* FMrcVideoCaptureUtils
 *****************************************************************************/

TArray<FMrcVideoCaptureFeedIndex> FMrcVideoCaptureUtils::EnumerateAvailableFeeds(UMediaPlayer* MediaPlayer)
{
	TArray<FMrcVideoCaptureFeedIndex> AvailableFormats;
	if (MediaPlayer != nullptr && !MediaPlayer->GetUrl().IsEmpty())
	{
		const int32 NumTracks = MediaPlayer->GetNumTracks(EMediaPlayerTrack::Video);
		for (int32 Track = 0; Track < NumTracks; ++Track)
		{
			const int32 FormatCount = MediaPlayer->GetNumTrackFormats(EMediaPlayerTrack::Video, Track);

			int32 BlockIndex = AvailableFormats.AddDefaulted(FormatCount);
			for (int32 FormatIndex = 0; FormatIndex < FormatCount; ++FormatIndex)
			{
				FMrcVideoCaptureFeedIndex& Format = AvailableFormats[BlockIndex + FormatIndex];
				Format.DeviceURL = MediaPlayer->GetUrl();
				Format.StreamIndex = Track;
				Format.FormatIndex = FormatIndex;
			}
		}
	}
	else
	{
		UE_LOGF(LogMixedRealityCapture, Warning, "Invalid media player for query - a valid, open capture feed is required for this query.");
	}
	return AvailableFormats;
}

bool FMrcVideoCaptureUtils::FeedSortPredicate(UMediaPlayer* MediaPlayer, const FMrcVideoCaptureFeedIndex& A, const FMrcVideoCaptureFeedIndex& B,
	const float PrioritizedAspectRatio, const int32 PrioritizedResolution, const FString& PreferedFormat)
{
	if (!PreferedFormat.IsEmpty())
	{
		const FString A_Format = MediaPlayer->GetVideoTrackType(A.StreamIndex, A.FormatIndex);
		const bool bAMatchesFormat = (A_Format == PreferedFormat);
		const FString B_Format = MediaPlayer->GetVideoTrackType(B.StreamIndex, B.FormatIndex);
		const bool bBMatchesFormat = (B_Format == PreferedFormat);

		if (bAMatchesFormat != bBMatchesFormat)
		{
			return bAMatchesFormat;
		}
	}

	const float A_AspectRatio = MediaPlayer->GetVideoTrackAspectRatio(A.StreamIndex, A.FormatIndex);
	const bool bAMatchesAspect = FMath::IsNearlyEqual(A_AspectRatio, PrioritizedAspectRatio);
	const float B_AspectRatio = MediaPlayer->GetVideoTrackAspectRatio(B.StreamIndex, B.FormatIndex);
	const bool bBMatchesAspect = FMath::IsNearlyEqual(B_AspectRatio, PrioritizedAspectRatio);

	// prioritize matching the aspect ratio
	if (bAMatchesAspect != bBMatchesAspect)
	{
		return bAMatchesAspect;
	}

	const FIntPoint A_Dim = MediaPlayer->GetVideoTrackDimensions(A.StreamIndex, A.FormatIndex);
	const bool bAMatchesRes = (A_Dim.Y >= PrioritizedResolution);
	const bool bAMatchesResExact = A_Dim.Y == PrioritizedResolution;
	const FIntPoint B_Dim = MediaPlayer->GetVideoTrackDimensions(B.StreamIndex, B.FormatIndex);
	const bool bBMatchesRes = (B_Dim.Y >= PrioritizedResolution);
	const bool bBMatchesResExact = B_Dim.Y == PrioritizedResolution;

	// next, order formats matching the desired resolution (equal and above)
	if (bAMatchesRes != bBMatchesRes)
	{
		return bAMatchesRes;
	}
	else if (!bAMatchesRes)
	{
		const int32 A_ScreenArea = A_Dim.X * A_Dim.Y;
		const int32 B_ScreenArea = B_Dim.X * B_Dim.Y;
		// if both resolutions are under what's desired, order them by screen coverage
		return A_ScreenArea > B_ScreenArea;
	}
	else if ((bAMatchesResExact || bBMatchesResExact) && (A_Dim.Y != B_Dim.Y))
	{
		return bAMatchesResExact;
	}

	const TRange<float> A_FrameRateRange = MediaPlayer->GetVideoTrackFrameRates(A.StreamIndex, A.FormatIndex);
	float A_FrameRate = MediaPlayer->GetVideoTrackFrameRate(A.StreamIndex, A.FormatIndex);
	if (!A_FrameRateRange.IsDegenerate() || A_FrameRateRange.GetLowerBoundValue() != A_FrameRate)
	{
		A_FrameRate = A_FrameRateRange.GetUpperBoundValue();
	}

	const TRange<float> B_FrameRateRange = MediaPlayer->GetVideoTrackFrameRates(B.StreamIndex, B.FormatIndex);
	float B_FrameRate = MediaPlayer->GetVideoTrackFrameRate(B.StreamIndex, B.FormatIndex);
	if (!B_FrameRateRange.IsDegenerate() || B_FrameRateRange.GetLowerBoundValue() != B_FrameRate)
	{
		B_FrameRate = B_FrameRateRange.GetUpperBoundValue();
	}

	// lastly, favor higher frame rates
	if (A_FrameRate != B_FrameRate)
	{
		return A_FrameRate > B_FrameRate;
	}
	if (A_FrameRateRange.GetLowerBoundValue() != B_FrameRateRange.GetLowerBoundValue())
	{
		return A_FrameRateRange.GetLowerBoundValue() > B_FrameRateRange.GetLowerBoundValue();
	}

	// maintain order if they're identical
	if (A.StreamIndex != B.StreamIndex)
	{
		return A.StreamIndex > B.StreamIndex;
	}
	return A.FormatIndex < B.FormatIndex;
}

static FMrcVideoCaptureFeedIndex FindPreferedCaptureFeed(UMediaPlayer* MediaPlayer)
{
	FMrcVideoCaptureFeedIndex BestCaptureFeed;

	TArray<FMrcVideoCaptureFeedIndex> FeedList = FMrcVideoCaptureUtils::EnumerateAvailableFeeds(MediaPlayer);
	if (FeedList.Num() > 0)
	{
		const TCHAR* MrcSettingsTag = TEXT("/Script/MixedRealityCaptureFramework.MixedRealityFrameworkSettings");

		FString DesiredFormat;
		GConfig->GetString(MrcSettingsTag, TEXT("DesiredCaptureFormat"), DesiredFormat, GEngineIni);
		float DesiredAspectRatio = 16.f / 9.f;
		GConfig->GetFloat(MrcSettingsTag, TEXT("DesiredCaptureAspectRatio"), DesiredAspectRatio, GEngineIni);
		int32 DesiredResolution = 1080;
		GConfig->GetInt(MrcSettingsTag, TEXT("DesiredCaptureResolution"), DesiredResolution, GEngineIni);

		FeedList.Sort([MediaPlayer, DesiredFormat, DesiredAspectRatio, DesiredResolution](const FMrcVideoCaptureFeedIndex& A, const FMrcVideoCaptureFeedIndex& B)->bool
		{
			return FMrcVideoCaptureUtils::FeedSortPredicate(MediaPlayer, A, B, DesiredAspectRatio, DesiredResolution, DesiredFormat);
		});

		BestCaptureFeed = FeedList[0];
	}

	return BestCaptureFeed;
}

/* FLatentPlayMrcCaptureFeedAction
*****************************************************************************/

struct FLatentPlayMrcCaptureFeedAction : public FTickableGameObject, public TSharedFromThis<FLatentPlayMrcCaptureFeedAction>
{
public:
	static TSharedRef<FLatentPlayMrcCaptureFeedAction> Create(UAsyncTask_OpenMrcVidCaptureFeedBase* Owner);
	static TSharedPtr<FLatentPlayMrcCaptureFeedAction> FindActiveAction(UMediaPlayer* MediaPlayer);
	static void FreeAction(UMediaPlayer* MediaPlayer);

	virtual ~FLatentPlayMrcCaptureFeedAction();
	UAsyncTask_OpenMrcVidCaptureFeedBase* GetOwner() const { return Owner; }

public:
	//~ FTickableObjectBase interface

	virtual bool IsTickable() const override;
	virtual void Tick(float DeltaTime) override;
	virtual TStatId GetStatId() const override;

private:
	FLatentPlayMrcCaptureFeedAction(UAsyncTask_OpenMrcVidCaptureFeedBase* Owner);
	
	void HandleMediaPlayerMediaEvent(EMediaEvent Event);
	void Destroy();

	UAsyncTask_OpenMrcVidCaptureFeedBase* Owner;
	FDelegateHandle OnMediaEventBinding;

	static TMap< UMediaPlayer*, TSharedPtr<FLatentPlayMrcCaptureFeedAction> > ActiveAsyncActions;
};
TMap< UMediaPlayer*, TSharedPtr<FLatentPlayMrcCaptureFeedAction> > FLatentPlayMrcCaptureFeedAction::ActiveAsyncActions;


TSharedRef<FLatentPlayMrcCaptureFeedAction> FLatentPlayMrcCaptureFeedAction::Create(UAsyncTask_OpenMrcVidCaptureFeedBase* InOwner)
{
	UMediaPlayer* MediaPlayer = InOwner->MediaPlayer;

	TSharedPtr<FLatentPlayMrcCaptureFeedAction> ExistingAction = FLatentPlayMrcCaptureFeedAction::FindActiveAction(MediaPlayer);
	if (ExistingAction.IsValid() && ExistingAction->GetOwner() == InOwner)
	{
		return ExistingAction.ToSharedRef();
	}
	else
	{
		FreeAction(MediaPlayer);

		TSharedRef<FLatentPlayMrcCaptureFeedAction> NewAction = MakeShareable(new FLatentPlayMrcCaptureFeedAction(InOwner));
		ActiveAsyncActions.Add(MediaPlayer, NewAction);

		return NewAction;
	}
}

TSharedPtr<FLatentPlayMrcCaptureFeedAction> FLatentPlayMrcCaptureFeedAction::FindActiveAction(UMediaPlayer* MediaPlayer)
{
	TSharedPtr<FLatentPlayMrcCaptureFeedAction> ActiveAction;
	if (TSharedPtr<FLatentPlayMrcCaptureFeedAction>* ExistingAction = ActiveAsyncActions.Find(MediaPlayer))
	{
		ActiveAction = *ExistingAction;
	}
	return ActiveAction;
}

void FLatentPlayMrcCaptureFeedAction::FreeAction(UMediaPlayer* MediaPlayer)
{
	TSharedPtr<FLatentPlayMrcCaptureFeedAction> ExistingAction = FLatentPlayMrcCaptureFeedAction::FindActiveAction(MediaPlayer);
	if (ExistingAction.IsValid())
	{
		MediaPlayer->OnMediaEvent().Remove(ExistingAction->OnMediaEventBinding);

		ActiveAsyncActions.Remove(MediaPlayer);
		ExistingAction->GetOwner()->CleanUp();
	}
}

FLatentPlayMrcCaptureFeedAction::FLatentPlayMrcCaptureFeedAction(UAsyncTask_OpenMrcVidCaptureFeedBase* InOwner)
	: Owner(InOwner)
{
	UMediaPlayer* MediaPlayer = InOwner->MediaPlayer;

	if (MediaPlayer)
	{
		OnMediaEventBinding = MediaPlayer->OnMediaEvent().AddRaw(this, &FLatentPlayMrcCaptureFeedAction::HandleMediaPlayerMediaEvent);
		MediaPlayer->Play();
	}
}

FLatentPlayMrcCaptureFeedAction::~FLatentPlayMrcCaptureFeedAction()
{
	UMediaPlayer* MediaPlayer = Owner->MediaPlayer;
	if (MediaPlayer)
	{
		MediaPlayer->OnMediaEvent().Remove(OnMediaEventBinding);
	}
}

bool FLatentPlayMrcCaptureFeedAction::IsTickable() const
{
	return Owner && Owner->MediaPlayer;
}

void FLatentPlayMrcCaptureFeedAction::Tick(float DeltaTime)
{
	bool bFinished = (Owner == nullptr);
	if (!bFinished)
	{
		UMediaPlayer* MediaPlayer = Owner->MediaPlayer;
		bFinished = (MediaPlayer == nullptr);

		if (!bFinished && (MediaPlayer->HasError() ))//|| !MediaPlayer->IsPlaying()/*|| IsStopped(), etc.*/))
		{
			FMrcVideoCaptureFeedIndex FailedFeedRef(MediaPlayer);
			Owner->OnFail.Broadcast(FailedFeedRef);

			bFinished = true;
		}
		else if (!bFinished && MediaPlayer->IsPlaying())
		{
			bFinished = true;
			for (UObject* BoundObj : Owner->OnFail.GetAllObjects())
			{
				if (IsValid(BoundObj))
				{
					bFinished = false;
					break;
				}
			}
		}
	}
	
	if (bFinished)
	{
		Destroy();
	}
}

TStatId FLatentPlayMrcCaptureFeedAction::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FLatentPlayMrcCaptureFeedAction, STATGROUP_ThreadPoolAsyncTasks);
}

void FLatentPlayMrcCaptureFeedAction::HandleMediaPlayerMediaEvent(EMediaEvent Event)
{
	switch (Event)
	{
	case EMediaEvent::MediaOpened:
		{
			if (Owner && Owner->MediaPlayer)
			{
				// on Win7, the WMF backend has to tear down and reopen when selecting the desired track/format
				// so here we ensure we kick the MediaPlayer back to playing (in case it is set to not play-on-open)
				Owner->MediaPlayer->Play();
			}
		}
		break;

	case EMediaEvent::MediaOpenFailed:
	case EMediaEvent::PlaybackEndReached:
		{
			if (Owner)
			{
				FMrcVideoCaptureFeedIndex FailedFeedRef(Owner->MediaPlayer);
				Owner->OnFail.Broadcast(FailedFeedRef);
			}
		}
	case EMediaEvent::MediaClosed:
		Destroy();
		break;

	default:
		break;
	}
}

void FLatentPlayMrcCaptureFeedAction::Destroy()
{
	for (auto& Action : ActiveAsyncActions)
	{
		if (Action.Value.Get() == this)
		{
			if (Owner)
			{
				if (UMediaPlayer* MediaPlayer = Owner->MediaPlayer)
				{
					MediaPlayer->OnMediaEvent().Remove(OnMediaEventBinding);
				}
				Owner->CleanUp();
			}
			ActiveAsyncActions.Remove(Action.Key);
			break;
		}
	}
}

/* UAsyncTask_OpenMrcVidCaptureFeedBase
 *****************************************************************************/

UAsyncTask_OpenMrcVidCaptureFeedBase::UAsyncTask_OpenMrcVidCaptureFeedBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bCachedPlayOnOpenVal(true)
{
	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		AddToRoot();
	}
}

void UAsyncTask_OpenMrcVidCaptureFeedBase::Open(UMediaPlayer* Target, const FString& DeviceURL)
{
	MediaPlayer = Target;
	// make sure nothing else is operating on this MediaPlayer
	FLatentPlayMrcCaptureFeedAction::FreeAction(Target);

	if (MediaPlayer)
	{
		MediaPlayer->OnMediaOpened.AddUniqueDynamic(this, &UAsyncTask_OpenMrcVidCaptureFeedBase::OnVideoFeedOpened);
		MediaPlayer->OnMediaOpenFailed.AddUniqueDynamic(this, &UAsyncTask_OpenMrcVidCaptureFeedBase::OnVideoFeedOpenFailure);

		bCachedPlayOnOpenVal = MediaPlayer->PlayOnOpen;
		MediaPlayer->PlayOnOpen = false;

		bool bHasError = MediaPlayer->HasError();

		MediaPlayer->Close();
		MediaPlayer->OpenUrl(DeviceURL);
	}
}

void UAsyncTask_OpenMrcVidCaptureFeedBase::OnVideoFeedOpened(FString DeviceUrl)
{
	FMrcVideoCaptureFeedIndex OpenedFeedRef(MediaPlayer);
	if (MediaPlayer)
	{
		LatentPlayer = FLatentPlayMrcCaptureFeedAction::Create(this);
		OnSuccess.Broadcast(OpenedFeedRef);

		// cannot remove, as we're likely iterating over this list
		//MediaPlayer->OnMediaOpened.RemoveDynamic(this, &UAsyncTask_OpenMrcVidCaptureFeedBase::OnVideoFeedOpened);
	}
	else
	{
		OnFail.Broadcast(OpenedFeedRef);
	}
}

void UAsyncTask_OpenMrcVidCaptureFeedBase::OnVideoFeedOpenFailure(FString DeviceUrl)
{
	CleanUp();

	FMrcVideoCaptureFeedIndex FailedFeedRef;
	FailedFeedRef.DeviceURL = DeviceUrl;
	FailedFeedRef.StreamIndex = INDEX_NONE;
	FailedFeedRef.FormatIndex = INDEX_NONE;

	OnFail.Broadcast(FailedFeedRef);
}

bool UAsyncTask_OpenMrcVidCaptureFeedBase::SetTrackFormat(const int32 StreamIndex, const int32 FormatIndex)
{
	bool bSuccess = false;
	if (MediaPlayer && !MediaPlayer->GetUrl().IsEmpty())
	{
		if (StreamIndex >= 0 && StreamIndex < MediaPlayer->GetNumTracks(EMediaPlayerTrack::Video))
		{
			const bool bSelected = MediaPlayer->GetSelectedTrack(EMediaPlayerTrack::Video) || MediaPlayer->SelectTrack(EMediaPlayerTrack::Video, StreamIndex);
			if (bSelected)
			{
				if (FormatIndex >= 0 && FormatIndex < MediaPlayer->GetNumTrackFormats(EMediaPlayerTrack::Video, StreamIndex))
				{
					bSuccess = (MediaPlayer->GetTrackFormat(EMediaPlayerTrack::Video, StreamIndex) == FormatIndex) || 
						MediaPlayer->SetTrackFormat(EMediaPlayerTrack::Video, StreamIndex, FormatIndex);
				}
			}
		}
	}
	return bSuccess;
}

void UAsyncTask_OpenMrcVidCaptureFeedBase::CleanUp()
{
	if (LatentPlayer.IsValid() && FLatentPlayMrcCaptureFeedAction::FindActiveAction(MediaPlayer) == LatentPlayer.Pin())
	{
		FLatentPlayMrcCaptureFeedAction::FreeAction(MediaPlayer);
	}
	LatentPlayer.Reset();

	if (MediaPlayer)
	{
		MediaPlayer->OnMediaOpenFailed.RemoveDynamic(this, &UAsyncTask_OpenMrcVidCaptureFeedBase::OnVideoFeedOpenFailure);
		MediaPlayer->OnMediaOpened.RemoveDynamic(this, &UAsyncTask_OpenMrcVidCaptureFeedBase::OnVideoFeedOpened);

		MediaPlayer->PlayOnOpen = bCachedPlayOnOpenVal;
		MediaPlayer = nullptr;
	}
	RemoveFromRoot();
	SetReadyToDestroy();
}

/* UAsyncTask_OpenMrcVidCaptureDevice
 *****************************************************************************/

UAsyncTask_OpenMrcVidCaptureDevice* UAsyncTask_OpenMrcVidCaptureDevice::OpenMrcVideoCaptureDevice(const FMediaCaptureDeviceInfo& DeviceId, UMediaPlayer* Target, FMRCaptureFeedDelegate::FDelegate OpenedCallback)
{
	UAsyncTask_OpenMrcVidCaptureDevice* OpenTask = NewObject<UAsyncTask_OpenMrcVidCaptureDevice>();
	if (OpenedCallback.IsBound())
	{
		OpenTask->OnSuccess.Add(OpenedCallback);
	}

	OpenTask->Open(Target, DeviceId.Url);
	return OpenTask;
}

UAsyncTask_OpenMrcVidCaptureDevice::UAsyncTask_OpenMrcVidCaptureDevice(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{}

void UAsyncTask_OpenMrcVidCaptureDevice::OnVideoFeedOpened(FString DeviceUrl)
{
	FMrcVideoCaptureFeedIndex BestFeed = FindPreferedCaptureFeed(MediaPlayer);
	if (ensure(DeviceUrl == BestFeed.DeviceURL))
	{
		SetTrackFormat(BestFeed.StreamIndex, BestFeed.FormatIndex);
	}

	Super::OnVideoFeedOpened(DeviceUrl);
}

/* UAsyncTask_OpenMrcVidCaptureFeed
 *****************************************************************************/

UAsyncTask_OpenMrcVidCaptureFeed* UAsyncTask_OpenMrcVidCaptureFeed::OpenMrcVideoCaptureFeed(const FMrcVideoCaptureFeedIndex& FeedRef, UMediaPlayer* Target, FMRCaptureFeedDelegate::FDelegate OpenedCallback)
{
	UAsyncTask_OpenMrcVidCaptureFeed* OpenTask = NewObject<UAsyncTask_OpenMrcVidCaptureFeed>();
	if (OpenedCallback.IsBound())
	{
		OpenTask->OnSuccess.Add(OpenedCallback);
	}

	OpenTask->Open(FeedRef, Target);
	return OpenTask;
}

UAsyncTask_OpenMrcVidCaptureFeed::UAsyncTask_OpenMrcVidCaptureFeed(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{}

void UAsyncTask_OpenMrcVidCaptureFeed::Open(const FMrcVideoCaptureFeedIndex& FeedRef, UMediaPlayer* Target)
{
	DesiredFeedRef = FeedRef;
	UAsyncTask_OpenMrcVidCaptureFeedBase::Open(Target, FeedRef.DeviceURL);
}

void UAsyncTask_OpenMrcVidCaptureFeed::OnVideoFeedOpened(FString DeviceUrl)
{
	if (ensure(DeviceUrl == DesiredFeedRef.DeviceURL))
	{
		SetTrackFormat(DesiredFeedRef.StreamIndex, DesiredFeedRef.FormatIndex);
	}
	else
	{
		FMrcVideoCaptureFeedIndex FallbackFeed = FindPreferedCaptureFeed(MediaPlayer);
		if (DeviceUrl == FallbackFeed.DeviceURL)
		{
			SetTrackFormat(FallbackFeed.StreamIndex, FallbackFeed.FormatIndex);
		}
	}
	Super::OnVideoFeedOpened(DeviceUrl);
}
