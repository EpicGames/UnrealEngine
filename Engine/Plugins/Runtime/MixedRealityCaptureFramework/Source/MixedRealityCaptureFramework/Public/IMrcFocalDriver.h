// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"

#include "IMrcFocalDriver.generated.h"


UINTERFACE(Category="MixedRealityCapture", Blueprintable, meta=(DisplayName = "Focal Driver"))
class MIXEDREALITYCAPTUREFRAMEWORK_API UMrcFocalDriver : public UInterface
{
public:
	GENERATED_BODY()
};

/** Interface that should be implemented to provide focal information such as field of view. */
class MIXEDREALITYCAPTUREFRAMEWORK_API IMrcFocalDriver
{
public:
	GENERATED_BODY()

public:
	
	/** Get the horizontal field of view of this provider. */
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "MixedRealityCapture", DisplayName = "GetHorizontalFieldOfView", meta = (CallInEditor = "true"))
	float GetHorizontalFieldOfView() const;
};
