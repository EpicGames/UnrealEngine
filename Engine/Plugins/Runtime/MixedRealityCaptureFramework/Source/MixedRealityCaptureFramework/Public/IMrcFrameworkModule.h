// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "HAL/IConsoleManager.h" // for TAutoConsoleVariable<>

class IConsoleVariable;

/**
 * 
 */
class IMrcFrameworkModule : public IModuleInterface
{
public:
	/** Virtual destructor. */
	virtual ~IMrcFrameworkModule() {}

	virtual class AMixedRealityCaptureActor* GetMixedRealityCaptureActor() { return nullptr; }
};

