// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "MixedRealityCaptureActor.generated.h"

class UMixedRealityCaptureComponent;
class USceneComponent;
class FMRCaptureAutoTargeter;
class UTexture;
enum class ESpectatorScreenMode : uint8;
class UStaticMesh;

/**
 *	
 */
UCLASS(Blueprintable, config=Engine)
class MIXEDREALITYCAPTUREFRAMEWORK_API AMixedRealityCaptureActor : public AActor
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category=MixedRealityCapture, meta=(AllowPrivateAccess="true"))
	TObjectPtr<UMixedRealityCaptureComponent> CaptureComponent;

	UPROPERTY(EditAnywhere, config, BlueprintReadOnly, Category=Tracking)
	bool bAutoAttachToVRPlayer;

	UPROPERTY(EditAnywhere, config, BlueprintReadOnly, Category=SceneCapture)
	bool bAutoHidePlayer;

	UPROPERTY(EditAnywhere, config, BlueprintReadOnly, Category=SceneCapture, meta=(editcondition="bAutoHidePlayer"))
	bool bHideAttachmentsWithPlayer;

	// If true the capture texture will automatically be applied to the Spectator Screen.
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, BlueprintSetter=SetAutoBroadcast, Category=SceneCapture)
	bool bAutoBroadcast;

public:
	//~ Blueprint API

	UFUNCTION(BlueprintSetter)
	void SetAutoBroadcast(const bool bNewValue);

	UFUNCTION(BlueprintPure, Category = SceneCapture)
	bool IsBroadcasting();

	UFUNCTION(BlueprintPure, Category = SceneCapture)
	UTexture* GetCaptureTexture();

public:
	bool SetTargetPlayer(APawn* PlayerPawn, USceneComponent* TrackingOrigin);

public:
	//~ AActor interface

	virtual void BeginPlay() override;
	virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;

private:
	UFUNCTION()
	void OnTargetDestroyed(AActor* DestroyedActor);

private:
	TWeakObjectPtr<APawn> TargetPlayer;
	TSharedPtr<FMRCaptureAutoTargeter> AutoTargeter;

	// @TODO: The SpectatorScreenController really should expose setting screen modes as a stack w/ a push/pop interface
	struct FCastingModeRestore
	{
	public:
		FCastingModeRestore();

		bool BeginCasting(UTexture* DisplayTexture);
		bool IsCasting() const { return bIsCasting; }
		void EndCasting();

	private:
		UTexture* RestoreTexture;
		ESpectatorScreenMode RestoreMode;
		bool bIsCasting;
	};
	FCastingModeRestore BroadcastManager;

	UPROPERTY(Transient)
	TObjectPtr<UStaticMesh> DebugVisualizerMesh;
};
