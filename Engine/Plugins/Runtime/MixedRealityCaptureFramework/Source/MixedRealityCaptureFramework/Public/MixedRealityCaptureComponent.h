// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SceneCaptureComponent2D.h"
#include "Delegates/Delegate.h"
#include "InputCoreTypes.h" // for EControllerHand
#include "Math/Color.h" // for FLinearColor
#include "MrcCalibrationData.h"
#include "MrcVideoCaptureDevice.h"
#include "OpenCVLensDistortionParameters.h"
#include "Templates/SubclassOf.h"

#include "MixedRealityCaptureComponent.generated.h"

class UMediaPlayer;
class UMaterial;
class UChildActorComponent;
class AMrcProjectionActor;
class UMotionControllerComponent;
class AStaticMeshActor;
class UMrcGarbageMatteCaptureComponent;
class UTextureRenderTarget2D;
class FMrcLatencyViewExtension;
class UMrcCalibrationData;
class AMrcGarbageMatteActor;
class IXRTrackingSystem;

DECLARE_LOG_CATEGORY_EXTERN(LogMixedRealityCapture, Log, All);

/**
 *	
 */
UCLASS(ClassGroup = Rendering, editinlinenew, Blueprintable, BlueprintType, config = Engine, meta = (BlueprintSpawnableComponent))
class MIXEDREALITYCAPTUREFRAMEWORK_API UMixedRealityCaptureComponent : public USceneCaptureComponent2D
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=VideoCapture)
	TObjectPtr<UMediaPlayer> MediaSource;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter=SetVidProjectionMat, Category=Composition)
	TObjectPtr<UMaterialInterface> VideoProcessingMaterial;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter=SetVidProcessingParams, Category=Composition)
	FMrcVideoProcessingParams VideoProcessingParams;

	UPROPERTY(BlueprintReadWrite, BlueprintSetter=SetCaptureDevice, Category=Composition)
	FMrcVideoCaptureFeedIndex CaptureFeedRef;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter=SetLensDistortionParameters, Category=VideoCapture)
	FOpenCVLensDistortionParameters LensDistortionParameters;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Tracking)
	FName TrackingSourceName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Composition)
	TObjectPtr<UTextureRenderTarget2D> GarbageMatteCaptureTextureTarget;

	/** Millisecond delay to apply to motion controller components when rendering to the capture view (to better align with latent camera feeds) */
	UPROPERTY(BlueprintReadWrite, Config, BlueprintGetter=GetTrackingDelay, BlueprintSetter=SetTrackingDelay, Category=Composition, meta=(ClampMin="0", UIMin="0"))
	int32 TrackingLatency;

	/** Determines if this component should attempt to load the default MR calibration file on initialization */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Config, Category=Calibration)
	bool bAutoLoadConfiguration;

	/** Depth offset (in UE units) for the card that the camera feed is projected onto. By default the card is aligned with the HMD. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, BlueprintSetter=SetProjectionDepthOffset, Category=Composition)
	float ProjectionDepthOffset;

	/** Enabled by default, the projection plane tracks with the HMD to simulate the depth of the player. Disable to keep the projection plane from moving. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, BlueprintSetter=SetEnableProjectionDepthTracking, Category=Tracking)
	bool bProjectionDepthTracking;

public:
	//~ UObject interface

	static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

public:
	//~ UActorComponent interface

	virtual void OnRegister() override;
	virtual void Activate(bool bReset = false) override;
	virtual void Deactivate() override;
	virtual void InitializeComponent() override;
	virtual void OnUpdateTransform(EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport) override;
	virtual void OnComponentDestroyed(bool bDestroyingHierarchy) override;

public:
	//~ USceneComponent interface

#if WITH_EDITOR
	virtual bool GetEditorPreviewInfo(float DeltaTime, FMinimalViewInfo& ViewOut) override;
#endif 

public:
	//~ USceneCaptureComponent interface

	virtual const AActor* GetViewOwner() const override;
	virtual void UpdateSceneCaptureContents(FSceneInterface* Scene, ISceneRenderBuilder& SceneRenderBuilder) override;

public:
	//~ Blueprint API	

	UFUNCTION(BlueprintCallable, Category = "MixedRealityCapture|Calibration", meta = (DisplayName = "SaveAsDefaultConfiguration"))
	bool SaveAsDefaultConfiguration_K2(); // non-const to make it an exec node
	bool SaveAsDefaultConfiguration() const;

	UFUNCTION(BlueprintCallable, Category = "MixedRealityCapture|Calibration", meta = (DisplayName="SaveConfiguration"))
	bool SaveConfiguration_K2(const FString& SlotName, int32 UserIndex); // non-const to make it an exec node
	bool SaveConfiguration(const FString& SlotName, int32 UserIndex) const;

	UFUNCTION(BlueprintCallable, Category = "MixedRealityCapture|Calibration")
	bool LoadDefaultConfiguration();

	UFUNCTION(BlueprintCallable, Category = "MixedRealityCapture|Calibration")
	bool LoadConfiguration(const FString& SlotName, int32 UserIndex);

	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "MixedRealityCapture|Calibration")
	UMrcCalibrationData* ConstructCalibrationData() const;

	UFUNCTION(BlueprintCallable, Category = "MixedRealityCapture|Calibration")
	void FillOutCalibrationData(UMrcCalibrationData* Dst) const;

	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "MixedRealityCapture|Calibration")
	void ApplyCalibrationData(UMrcCalibrationData* ConfigData);

	/**
	 * Set an external garbage matte actor to be used instead of the mixed reality component's
	 * normal configuration save game based actor.  This is used during garbage matte setup to
	 * preview the garbage mask in realtime.
	 */
	UFUNCTION(BlueprintCallable, Category = "MixedRealityCapture|GarbageMatting")
	bool SetGarbageMatteActor(AMrcGarbageMatteActor* Actor);

	UFUNCTION(BlueprintSetter)
	void SetVidProjectionMat(UMaterialInterface* NewMaterial);

	UFUNCTION(BlueprintSetter)
	void SetVidProcessingParams(const FMrcVideoProcessingParams& NewVidProcessingParams);

	UFUNCTION(BlueprintCallable, Category = "MixedRealityCapture|Tracking")
	void SetDeviceAttachment(FName SourceName);

	UFUNCTION(BlueprintCallable, Category = "MixedRealityCapture|Tracking")
	void DetatchFromDevice();

	UFUNCTION(BlueprintCallable, Category = "MixedRealityCapture|Tracking")
	bool IsTracked() const;

	UFUNCTION(BlueprintSetter)
	void SetCaptureDevice(const FMrcVideoCaptureFeedIndex& FeedRef);

	UFUNCTION(BlueprintSetter)
	void SetLensDistortionParameters(const FOpenCVLensDistortionParameters& ModelRef);

	UFUNCTION(BlueprintGetter)
	int32 GetTrackingDelay() const;

	UFUNCTION(BlueprintSetter)
	void SetTrackingDelay(int32 DelayMS);

	UFUNCTION(BlueprintSetter)
	void SetProjectionDepthOffset(float DepthOffset);

	UFUNCTION(BlueprintPure, Category = "MixedRealityCapture|Projection", meta=(DisplayName="GetProjectionActor"))
	AActor* GetProjectionActor_K2() const;
	AMrcProjectionActor* GetProjectionActor() const;

	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FMRCaptureFeedOpenedDelegate, const FMrcVideoCaptureFeedIndex&, FeedRef);
	UPROPERTY(BlueprintAssignable, Category = VideoCapture)
	FMRCaptureFeedOpenedDelegate OnCaptureSourceOpened;

	/** 
	 * Enabled by default, the projection plane tracks with the HMD to simulate 
	 * the depth of the player. Disable to keep the projection plane from moving.
	 */
	UFUNCTION(BlueprintSetter)
	void SetEnableProjectionDepthTracking(bool bEnable = true);

public:
	/**
	 *
	 */
	void RefreshCameraFeed();

	/**
	 *
	 */
	void RefreshDevicePairing();

	/**
	 *
	 */
	void RefreshTrackingOriginOffset();


private:
	UFUNCTION() // needs to be a UFunction for binding purposes
	void OnVideoFeedOpened(const FMrcVideoCaptureFeedIndex& FeedRef);

	/**
	 *
	 */
	void RefreshProjectionDimensions();

	/**
	 *
	 */
	float GetDesiredAspectRatio() const;

	/**
	 *
	 */
	void RefreshDistortionDisplacementMap();

	/**
	 *
	 */
	void RefreshFOV();

	/**
	 *
	 */
	void OnTrackingOriginChanged(const IXRTrackingSystem* TrackingSys);

private:
#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient)
	TObjectPtr<UStaticMesh> ProxyMesh;

	UPROPERTY(Transient)
	TObjectPtr<UStaticMeshComponent> ProxyMeshComponent;
#endif

	UPROPERTY(Transient)
	TObjectPtr<UChildActorComponent> ProjectionActor;

	UPROPERTY(Transient)
	TObjectPtr<UMotionControllerComponent> PairedTracker;

	UPROPERTY(Transient)
	TObjectPtr<USceneComponent> TrackingOriginOffset;

	EHMDTrackingOrigin::Type RelativeOriginType;

	UPROPERTY(Transient)
	TObjectPtr<UMrcGarbageMatteCaptureComponent> GarbageMatteCaptureComponent;

	UPROPERTY(Transient)
	TObjectPtr<UTexture2D> DistortionDisplacementMap;

	float CalibratedFOV;
	FOpenCVCameraViewInfo UndistortedCameraInfo;

	TSharedPtr<FMrcLatencyViewExtension, ESPMode::ThreadSafe> ViewExtension;
};
