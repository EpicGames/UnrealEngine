// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/SaveGame.h"
#include "HeadMountedDisplayTypes.h" // for EHMDTrackingOrigin
#include "Math/Color.h" // for FLinearColor
#include "MrcVideoCaptureDevice.h" // for FMrcVideoCaptureFeedIndex
#include "OpenCVLensDistortionParameters.h" // for FOpenCVLensDistortionParameters 

#include "MrcCalibrationData.generated.h"

class UMaterialInstanceDynamic;

USTRUCT(BlueprintType)
struct FMrcLensCalibrationData
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(BlueprintReadWrite, Category = Data)
	float FOV = 90.f;

	UPROPERTY(BlueprintReadWrite, Category = Data)
	FOpenCVLensDistortionParameters DistortionParameters;
};

USTRUCT(BlueprintType)
struct FMrcAlignmentSaveData
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(BlueprintReadWrite, Category = Data)
	FVector CameraOrigin = FVector::ZeroVector;

	UPROPERTY(BlueprintReadWrite, Category = Data)
	FRotator Orientation = FRotator::ZeroRotator;

	UPROPERTY(BlueprintReadWrite, Category = Data)
	FName TrackingAttachmentId;

	UPROPERTY(BlueprintReadWrite, Category = Data)
	TEnumAsByte<EHMDTrackingOrigin::Type> TrackingOrigin;
};

USTRUCT(BlueprintType)
struct FMrcGarbageMatteSaveData
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(BlueprintReadWrite, Category = Data)
	FTransform Transform;
};

USTRUCT(BlueprintType)
struct FMrcVideoProcessingParams
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(BlueprintReadWrite, Category = Data)
	TMap<FName, float> MaterialScalarParams;

	UPROPERTY(BlueprintReadWrite, Category = Data)
	TMap<FName, FLinearColor> MaterialVectorParams;
};

USTRUCT(BlueprintType)
struct FMrcCompositingSaveData
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(BlueprintReadWrite, Category = Data)
	FMrcVideoCaptureFeedIndex CaptureDeviceURL;

	UPROPERTY(BlueprintReadWrite, Category = Data)
	float DepthOffset = 0.0f;

	UPROPERTY(BlueprintReadWrite, Category = Data)
	int32 TrackingLatency = 0;

	UPROPERTY(BlueprintReadWrite, Category = Data)
	FMrcVideoProcessingParams VideoProcessingParams;
};

UCLASS(BlueprintType, Blueprintable, config = Engine)
class MIXEDREALITYCAPTUREFRAMEWORK_API UMrcCalibrationData : public USaveGame
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY(BlueprintReadWrite, Category = SaveData)
	FMrcLensCalibrationData LensData;

	UPROPERTY(BlueprintReadWrite, Category = SaveData)
	FMrcAlignmentSaveData AlignmentData;

	UPROPERTY(BlueprintReadWrite, Category = SaveData)
	TArray<FMrcGarbageMatteSaveData> GarbageMatteSaveDatas;

	UPROPERTY(BlueprintReadWrite, Category = SaveData)
	FMrcCompositingSaveData CompositingData;
};

/**
 * 
 */
UCLASS(BlueprintType, config = Engine)
class MIXEDREALITYCAPTUREFRAMEWORK_API UMrcCalibrationSaveGame : public UMrcCalibrationData
{
	GENERATED_UCLASS_BODY()

public:
	// Metadata about the save file

	UPROPERTY(BlueprintReadWrite, config, Category = SaveMetadata)
	FString SaveSlotName;

	UPROPERTY(BlueprintReadWrite, config, Category = SaveMetadata)
	int32 UserIndex;
	
	UPROPERTY(BlueprintReadOnly, Category = SaveMetadata)
	int32 ConfigurationSaveVersion;
};