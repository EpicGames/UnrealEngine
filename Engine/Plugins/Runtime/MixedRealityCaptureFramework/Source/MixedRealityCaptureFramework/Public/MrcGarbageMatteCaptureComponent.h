// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SceneCaptureComponent2D.h"
#include "GameFramework/Actor.h"
#include "MrcCalibrationData.h" // for FMrcGarbageMatteSaveData
#include "MrcGarbageMatteCaptureComponent.generated.h"

class UMrcCalibrationData;
class UMrcFocalDriver;
class AMrcGarbageMatteActor;

/**
 *	
 */
UCLASS(ClassGroup = Rendering, editinlinenew, config = Engine, meta=(BlueprintSpawnableComponent))
class MIXEDREALITYCAPTUREFRAMEWORK_API UMrcGarbageMatteCaptureComponent : public USceneCaptureComponent2D
{
	GENERATED_UCLASS_BODY()

public:
	//~ UActorComponent interface
	virtual void OnComponentDestroyed(bool bDestroyingHierarchy) override;
	virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;

public:
	//~ USceneCaptureComponent interface
	virtual const AActor* GetViewOwner() const override;

public:
	UFUNCTION()
	void SetTrackingOrigin(USceneComponent* TrackingOrigin);

	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "MixedRealityCapture|GarbageMatting")
	void ApplyCalibrationData(const UMrcCalibrationData* ConfigData);

	UFUNCTION(BlueprintCallable, Category = "MixedRealityCapture|GarbageMatting")
	void SetGarbageMatteActor(AMrcGarbageMatteActor* NewActor);

	UFUNCTION(BlueprintCallable, Category = "MixedRealityCapture|GarbageMatting")
	void GetGarbageMatteData(TArray<FMrcGarbageMatteSaveData>& GarbageMatteDataOut);

	UFUNCTION(BlueprintCallable, Category = "MixedRealityCapture|GarbageMatting")
	void SetFocalDriver(TScriptInterface<IMrcFocalDriver> InFocalDriver);

	UFUNCTION(BlueprintNativeEvent, Category = "MixedRealityCapture|GarbageMatting")
	AMrcGarbageMatteActor* SpawnNewGarbageMatteActor(USceneComponent* TrackingOrigin);

private:
	void CleanupSpawnedActors();

	void PollFocalDriver();

private:
	UPROPERTY(Transient, Config)
	TSubclassOf<AMrcGarbageMatteActor> GarbageMatteActorClass;

	UPROPERTY(Transient)
	TObjectPtr<AMrcGarbageMatteActor> GarbageMatteActor;

	UPROPERTY(Transient)
	TArray<TObjectPtr<AMrcGarbageMatteActor>> SpawnedActors;

	UPROPERTY(Transient)
	TWeakObjectPtr<USceneComponent> TrackingOriginPtr;
	
	UPROPERTY(Transient)
	TScriptInterface<IMrcFocalDriver> FocalDriver;
};

/* AMrcGarbageMatteActor
 *****************************************************************************/

class UMaterial;
class UStaticMesh;

UCLASS(Blueprintable)
class AMrcGarbageMatteActor : public AActor
{
	GENERATED_UCLASS_BODY()


public: 
	UFUNCTION(BlueprintCallable, Category = "MixedRealityCapture|GarbageMatting")
	void ApplyCalibrationData(const TArray<FMrcGarbageMatteSaveData>& GarbageMatteData);

	UFUNCTION(BlueprintCallable, Category = "MixedRealityCapture|GarbageMatting")
	UPrimitiveComponent* AddNewGabageMatte(const FMrcGarbageMatteSaveData& GarbageMatteData);

	UFUNCTION(BlueprintNativeEvent, Category = "MixedRealityCapture|GarbageMatting")
	UPrimitiveComponent* CreateGarbageMatte(const FMrcGarbageMatteSaveData& GarbageMatteData);

	UFUNCTION(BlueprintCallable, Category = "MixedRealityCapture|GarbageMatting")
	void GetGarbageMatteData(TArray<FMrcGarbageMatteSaveData>& GarbageMatteDataOut);

private:
	UPROPERTY(Transient)
	TObjectPtr<UStaticMesh> GarbageMatteMesh;

	UPROPERTY(Transient)
	TObjectPtr<UMaterial> GarbageMatteMaterial;

	UPROPERTY(Transient, BlueprintReadOnly, Category = "MixedRealityCapture|GarbageMatting", meta=(AllowPrivateAccess="true"))
	TArray<TObjectPtr<UPrimitiveComponent>> GarbageMattes;
};
