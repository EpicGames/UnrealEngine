// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "MrcUtilLibrary.generated.h"

class UCameraComponent;
class APawn;
class APlayerController;
class UMaterialBillboardComponent;

UCLASS()
class MIXEDREALITYCAPTUREFRAMEWORK_API UMrcUtilLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

public:
	/**
	 * Return true if the system is sending the capture texture to the spectator screen.
	 */	
	UFUNCTION(BlueprintPure, Category = "MixedRealityCapture|Utils")
	static bool IsMixedRealityCaptureBroadcasting();

	/**
	 * Toggle whether the capture system is sending the capture texture to the spectator screen or not.
	 */
	UFUNCTION(BlueprintCallable, Category = "MixedRealityCapture|Utils")
	static void SetMixedRealityCaptureBroadcasting(bool bEnable);

	/**
	 * Return the Capture Texture, or nullptr if there isn't one.
	 */	
	UFUNCTION(BlueprintPure, Category = "MixedRealityCapture|Utils")
	static UTexture* GetMixedRealityCaptureTexture();

public:
	static USceneComponent* FindAssociatedHMDRoot(AActor* ActorInst);
	static USceneComponent* GetHMDRootComponent(const APawn* PlayerPawn);

private:
	static APawn* FindAssociatedPlayerPawn(AActor* ActorInst);
	static UCameraComponent* GetHMDCameraComponent(const APawn* PlayerPawn);
	static USceneComponent* GetHMDRootComponent(const UObject* WorldContextObject, int32 PlayerIndex);
};
