// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Kismet/BlueprintAsyncActionBase.h"
#include "MrcVideoCaptureDevice.generated.h"

class  UMediaPlayer;
struct FMediaCaptureDeviceInfo;

/* FMrcVideoCaptureFeedIndex
 *****************************************************************************/

USTRUCT(BlueprintType)
struct MIXEDREALITYCAPTUREFRAMEWORK_API FMrcVideoCaptureFeedIndex
{
	GENERATED_USTRUCT_BODY()

public:
	FMrcVideoCaptureFeedIndex();
	FMrcVideoCaptureFeedIndex(UMediaPlayer* MediaPlayer);
	FMrcVideoCaptureFeedIndex(FMediaCaptureDeviceInfo& DeviceInfo);

	UPROPERTY(BlueprintReadWrite, Category="MixedRealityCapture|CaptureDevice")
	FString DeviceURL;
	UPROPERTY(BlueprintReadWrite, Category="MixedRealityCapture|CaptureDevice")
	int32 StreamIndex;
	UPROPERTY(BlueprintReadWrite, Category="MixedRealityCapture|CaptureDevice")
	int32 FormatIndex;

	bool IsSet(UMediaPlayer* MediaPlayer) const;
	bool IsDeviceUrlValid() const;

	bool operator==(const FMrcVideoCaptureFeedIndex& Rhs) const;
	bool operator!=(const FMrcVideoCaptureFeedIndex& Rhs) const;
};

/* FMrcVideoCaptureUtils
 *****************************************************************************/

class MIXEDREALITYCAPTUREFRAMEWORK_API FMrcVideoCaptureUtils
{
public:
	static TArray<FMrcVideoCaptureFeedIndex> EnumerateAvailableFeeds(UMediaPlayer* MediaPlayer);

	static bool FeedSortPredicate(UMediaPlayer* MediaPlayer, const FMrcVideoCaptureFeedIndex& Rhs, const FMrcVideoCaptureFeedIndex& Lhs,
		const float    PrioritizedAspectRatio = 16/9.f,
		const int32    PrioritizedResolution  = 1080,
		const FString& PreferedFormat         = FString());
};

/* UAsyncTask_OpenMrcVidCaptureFeedBase
 *****************************************************************************/

struct FLatentPlayMrcCaptureFeedAction;
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FMRCaptureFeedDelegate, const FMrcVideoCaptureFeedIndex&, FeedRef);

UCLASS(Abstract)
class MIXEDREALITYCAPTUREFRAMEWORK_API UAsyncTask_OpenMrcVidCaptureFeedBase : public UBlueprintAsyncActionBase
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY(BlueprintAssignable)
	FMRCaptureFeedDelegate OnSuccess;

	UPROPERTY(BlueprintAssignable)
	FMRCaptureFeedDelegate OnFail;

public:
	void Open(UMediaPlayer* Target, const FString& DeviceURL);

protected:
	UFUNCTION()
	virtual void OnVideoFeedOpened(FString DeviceUrl);

	UFUNCTION()
	virtual void OnVideoFeedOpenFailure(FString DeviceUrl);

	bool SetTrackFormat(const int32 StreamIndex, const int32 FormatIndex);
	void CleanUp();

	UPROPERTY()
	TObjectPtr<UMediaPlayer> MediaPlayer;
	
private:
	friend struct FLatentPlayMrcCaptureFeedAction;
	bool bCachedPlayOnOpenVal;

	TWeakPtr<FLatentPlayMrcCaptureFeedAction> LatentPlayer;
};

/* UAsyncTask_OpenMrcVidCaptureDevice
 *****************************************************************************/

UCLASS()
class MIXEDREALITYCAPTUREFRAMEWORK_API UAsyncTask_OpenMrcVidCaptureDevice : public UAsyncTask_OpenMrcVidCaptureFeedBase
{
	GENERATED_UCLASS_BODY()

public:
	static UAsyncTask_OpenMrcVidCaptureDevice* OpenMrcVideoCaptureDevice(const FMediaCaptureDeviceInfo& DeviceId, UMediaPlayer* Target, FMRCaptureFeedDelegate::FDelegate OpenedCallback);

private:
	virtual void OnVideoFeedOpened(FString DeviceUrl) override;
};

/* UAsyncTask_OpenMrcVidCaptureFeed
 *****************************************************************************/

UCLASS()
class MIXEDREALITYCAPTUREFRAMEWORK_API UAsyncTask_OpenMrcVidCaptureFeed : public UAsyncTask_OpenMrcVidCaptureFeedBase
{
	GENERATED_UCLASS_BODY()

public:
	static UAsyncTask_OpenMrcVidCaptureFeed* OpenMrcVideoCaptureFeed(const FMrcVideoCaptureFeedIndex& FeedRef, UMediaPlayer* Target, FMRCaptureFeedDelegate::FDelegate OpenedCallback);

protected:
	void Open(const FMrcVideoCaptureFeedIndex& FeedRef, UMediaPlayer* Target);

private:
	virtual void OnVideoFeedOpened(FString DeviceUrl) override;

	FMrcVideoCaptureFeedIndex DesiredFeedRef;
};
